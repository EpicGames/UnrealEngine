{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Playlists",
	"Description": "Sequencer Playlists allow users to prepare, queue, and trigger level sequences on the fly during a virtual production session, providing increased flexibility and agility when interacting with animation on set.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "SequencerPlaylists",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}