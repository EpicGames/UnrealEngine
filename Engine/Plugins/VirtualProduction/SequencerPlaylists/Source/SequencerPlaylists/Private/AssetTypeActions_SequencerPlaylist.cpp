// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_SequencerPlaylist.h"

#include "SequencerPlaylist.h"
#include "SequencerPlaylistsModule.h"



UClass* FAssetTypeActions_SequencerPlaylist::GetSupportedClass() const
{
	return USequencerPlaylist::StaticClass();
}

void FAssetTypeActions_SequencerPlaylist::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> InEditWithinLevelEditor /* = TSharedPtr<IToolkitHost>() */)
{
	for (UObject* PlaylistObj : InObjects)
	{
		if (USequencerPlaylist* Playlist = Cast<USequencerPlaylist>(PlaylistObj))
		{
			FSequencerPlaylistsModule& Module = static_cast<FSequencerPlaylistsModule&>(FSequencerPlaylistsModule::Get());
			Module.SpawnEditorForPlaylist(Playlist);
		}
	}
}
