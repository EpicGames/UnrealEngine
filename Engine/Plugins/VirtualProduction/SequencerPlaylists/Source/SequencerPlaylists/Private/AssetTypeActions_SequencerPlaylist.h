// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

class FAssetTypeActions_SequencerPlaylist
	: public FAssetTypeActions_Base
{
public:
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SequencerPlaylist", "Sequencer Playlist"); }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::None; }
	virtual FColor GetTypeColor() const override { return FColor(0, 232, 0); }
	virtual UClass* GetSupportedClass() const override;
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> InEditWithinLevelEditor /* = TSharedPtr<IToolkitHost>() */) override;
};
