// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerPlaylistItem_Sequence.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Framework/Notifications/NotificationManager.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "Recorder/TakeRecorder.h"
#include "SequencerPlaylist.h"
#include "SequencerPlaylistsSubsystem.h"
#include "TrackEditors/SubTrackEditorBase.h" // for FSubTrackEditorUtil::CanAddSubSequence
#include "Tracks/MovieSceneSubTrack.h"
#include "Widgets/Notifications/SNotificationList.h"


#define LOCTEXT_NAMESPACE "SequencerPlaylists"


namespace UE::Private::SequencerPlaylistItem_Sequence
{
	bool IsTimeWithinSection(FQualifiedFrameTime InTime, UMovieSceneSection* Section)
	{
		check(Section);

		UMovieScene* SectionScene = Section->GetTypedOuter<UMovieScene>();
		check(SectionScene);

		const FFrameTime Time_SceneTicks = InTime.ConvertTo(SectionScene->GetTickResolution());
		return Section->IsTimeWithinSection(Time_SceneTicks.FloorToFrame());
	}


	UMovieSceneSubSection* IsTimeWithinAnySection(FQualifiedFrameTime InTime, TArrayView<TWeakObjectPtr<UMovieSceneSubSection>> WeakSectionArray)
	{
		for (TWeakObjectPtr<UMovieSceneSubSection> WeakSection : WeakSectionArray)
		{
			if (UMovieSceneSubSection* Section = WeakSection.Get())
			{
				if (IsTimeWithinSection(InTime, Section))
				{
					return Section;
				}
			}
		}

		return nullptr;
	}
}


/*static*/ FName USequencerPlaylistItem_Sequence::GetSequencePropertyName()
{
	static const FName SequencePropertyName =
		GET_MEMBER_NAME_CHECKED(USequencerPlaylistItem_Sequence, Sequence);

	return SequencePropertyName;
}


FText USequencerPlaylistItem_Sequence::GetDisplayName()
{
	return Sequence ? Sequence->GetDisplayName() : LOCTEXT("SequenceItemNullDisplayName", "(No sequence)");
}


void USequencerPlaylistItem_Sequence::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName ChangedPropertyName = PropertyChangedEvent.MemberProperty
		? PropertyChangedEvent.MemberProperty->GetFName()
		: NAME_None;

	if (ChangedPropertyName == NAME_None || ChangedPropertyName == GetSequencePropertyName())
	{
		USequencerPlaylistsSubsystem* Subsystem =
			GEditor->GetEditorSubsystem<USequencerPlaylistsSubsystem>();
		if (Subsystem)
		{
			Subsystem->UpdatePreloadSet();
		}
	}
}


void USequencerPlaylistItem_Sequence::SetSequence(ULevelSequence* NewSequence)
{
	Sequence = NewSequence;

	if (USequencerPlaylistsSubsystem* Subsystem = GEditor->GetEditorSubsystem<USequencerPlaylistsSubsystem>())
	{
		Subsystem->UpdatePreloadSet();
	}
}


FSequencerPlaylistItemPlayer_Sequence::FSequencerPlaylistItemPlayer_Sequence(TSharedRef<ISequencer> Sequencer)
	: WeakSequencer(Sequencer)
{
	Sequencer->OnStopEvent().AddRaw(this, &FSequencerPlaylistItemPlayer_Sequence::ClearItemStates);
}


FSequencerPlaylistItemPlayer_Sequence::~FSequencerPlaylistItemPlayer_Sequence()
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->OnStopEvent().RemoveAll(this);
	}
}


bool FSequencerPlaylistItemPlayer_Sequence::Play(
	USequencerPlaylistItem* Item,
	ESequencerPlaylistPlaybackDirection Direction // = ESequencerPlaylistPlaybackDirection::Forward
)
{
	FPlayParams PlayParams;
	PlayParams.Direction = Direction;
	return InternalPlay(Item, PlayParams);
}


bool FSequencerPlaylistItemPlayer_Sequence::TogglePause(USequencerPlaylistItem* Item)
{
	return InternalPause(Item);
}


bool FSequencerPlaylistItemPlayer_Sequence::Stop(USequencerPlaylistItem* Item)
{
	return InternalStop(Item);
}


bool FSequencerPlaylistItemPlayer_Sequence::AddHold(USequencerPlaylistItem* Item)
{
	return InternalAddHold(Item, FHoldParams());
}


bool FSequencerPlaylistItemPlayer_Sequence::Reset(USequencerPlaylistItem* Item)
{
	return InternalReset(Item);
}


FSequencerPlaylistPlaybackState
FSequencerPlaylistItemPlayer_Sequence::GetPlaybackState(USequencerPlaylistItem* Item) const
{
	FSequencerPlaylistPlaybackState Result;

	USequencerPlaylistItem_Sequence* SequenceItem = CastChecked<USequencerPlaylistItem_Sequence>(Item);
	if (!SequenceItem || !SequenceItem->GetSequence())
	{
		return Result;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return Result;
	}

	if (IsSequencerRecordingOrPlaying())
	{
		const FItemState* ItemState = ItemStates.Find(Item);
		if (!ItemState)
		{
			return Result;
		}

		Result.bIsPlaying = ItemState->PlayingUntil_RootTicks > Sequencer->GetGlobalTime().Time.FloorToFrame().Value;
		Result.bIsPaused = ItemState->WeakHoldSection.IsValid();
		Result.PlaybackDirection = ItemState->LastPlayDirection;
	}
	else
	{
		Result.bIsPaused = Item->bHoldAtFirstFrame;
	}

	return Result;
}


bool FSequencerPlaylistItemPlayer_Sequence::InternalPlay(
	USequencerPlaylistItem* Item,
	const FPlayParams& PlayParams
)
{
	USequencerPlaylistItem_Sequence* SequenceItem = CastChecked<USequencerPlaylistItem_Sequence>(Item);
	if (!SequenceItem || !SequenceItem->GetSequence())
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return false;
	}

	ULevelSequence* RootSequence = Cast<ULevelSequence>(Sequencer->GetRootMovieSceneSequence());
	UMovieScene* RootScene = RootSequence->GetMovieScene();

	if (!FSubTrackEditorUtil::CanAddSubSequence(RootSequence, *SequenceItem->GetSequence()))
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("InvalidSequence", "Invalid level sequence {0}. There could be a circular dependency."), SequenceItem->GetSequence()->GetDisplayName()));
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info);
		return false;
	}

	RootSequence->Modify();

	FItemState& ItemState = ItemStates.FindOrAdd(Item);
	UMovieSceneSubTrack* WorkingTrack = GetOrCreateWorkingTrack(Item);

	if (UMovieSceneSubSection* HoldSection = ItemState.WeakHoldSection.Get())
	{
		EndSection(HoldSection);
		TruncatePlayingUntil(ItemState);
		ItemState.WeakHoldSection.Reset();
	}

	ItemState.LastPlayDirection = PlayParams.Direction;

	UMovieScene* PlayScene = SequenceItem->GetSequence()->GetMovieScene();
	const TRange<FFrameNumber> SequencePlayRange = PlayScene->GetPlaybackRange();

	const FFrameTime StartFrameOffset_Ticks = PlayParams.StartFrameOffset_SceneTicks.IsSet()
		? PlayParams.StartFrameOffset_SceneTicks.GetValue()
		: ConvertFrameTime(FFrameNumber(SequenceItem->StartFrameOffset),
		                   PlayScene->GetDisplayRate(), PlayScene->GetTickResolution());

	const FFrameTime EndFrameOffset_Ticks = PlayParams.EndFrameOffset_SceneTicks.IsSet()
		? PlayParams.EndFrameOffset_SceneTicks.GetValue()
		: ConvertFrameTime(FFrameNumber(SequenceItem->EndFrameOffset),
		                   PlayScene->GetDisplayRate(), PlayScene->GetTickResolution());

	TRange<FFrameNumber> SectionRange(
		SequencePlayRange.GetLowerBoundValue() + StartFrameOffset_Ticks.FloorToFrame(),
		SequencePlayRange.GetUpperBoundValue() - EndFrameOffset_Ticks.FloorToFrame());

	const float TimeScale = FMath::Max(SMALL_NUMBER, SequenceItem->PlaybackSpeed);
	const FFrameTime SingleLoopDuration = ConvertFrameTime(SectionRange.Size<FFrameTime>() / TimeScale,
		PlayScene->GetTickResolution(), RootScene->GetTickResolution());

	const FFrameNumber NowFrame = Sequencer->GetGlobalTime().Time.FloorToFrame();
	const FFrameNumber EnqueueFrame = FMath::Max(ItemState.PlayingUntil_RootTicks, NowFrame.Value);
	const FFrameNumber StartFrame = PlayParams.bEnqueueExtraPlays ? EnqueueFrame : NowFrame;

	const int32 MinDuration = 1;
	const int32 MaxDuration = TNumericLimits<int32>::Max() - StartFrame.Value - 1;
	const int32 Duration = FMath::Clamp(
		(SingleLoopDuration * FMath::Max(1, SequenceItem->NumLoops + 1)).FloorToFrame().Value,
		MinDuration, MaxDuration);

	ItemState.PlayingUntil_RootTicks = FMath::Max(StartFrame.Value + Duration, ItemState.PlayingUntil_RootTicks);

	UMovieSceneSubSection* WorkingSubSection = WorkingTrack->AddSequence(SequenceItem->GetSequence(), StartFrame, Duration);

	WorkingSubSection->Parameters.TimeScale = TimeScale;
	WorkingSubSection->Parameters.StartFrameOffset = StartFrameOffset_Ticks.FloorToFrame();
	WorkingSubSection->Parameters.EndFrameOffset = EndFrameOffset_Ticks.FloorToFrame();

	if (SequenceItem->NumLoops != 0)
	{
		WorkingSubSection->Parameters.bCanLoop = true;
	}

	if (PlayParams.Direction == ESequencerPlaylistPlaybackDirection::Reverse)
	{
		WorkingSubSection->Parameters.TimeScale.ScaleBy(-1.0f);

		// Reverse playback with bCanLoop=true makes the engine add NegativeRateOffset=Duration to the inner start offset, which lines the loop
		// boundary up exactly with the section's first sample. LoopTime then mod-wraps the first iteration to a single frame and the take appears not
		// to play. Offsetting the first loop by one full iteration cancels that alignment so every iteration plays the take end-to-start.
		// 
		WorkingSubSection->Parameters.FirstLoopStartFrameOffset = -SingleLoopDuration.FloorToFrame().Value;
	}

	ItemState.WeakPlaySections.Add(WorkingSubSection);

	return true;
}


bool FSequencerPlaylistItemPlayer_Sequence::InternalPause(USequencerPlaylistItem* Item)
{
	using namespace UE::Private::SequencerPlaylistItem_Sequence;

	bool bSequenceWasModified = false;

	USequencerPlaylistItem_Sequence* SequenceItem = CastChecked<USequencerPlaylistItem_Sequence>(Item);
	if (!SequenceItem || !SequenceItem->GetSequence())
	{
		return bSequenceWasModified;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return bSequenceWasModified;
	}

	if (IsSequencerRecordingOrPlaying())
	{
		FItemState& ItemState = ItemStates.FindOrAdd(Item);
		if (UMovieSceneSubSection* HoldSection = ItemState.WeakHoldSection.Get())
		{
			// Resume playback
			FPlayParams PlayParams;
			PlayParams.Direction = ItemState.LastPlayDirection;
			PlayParams.StartFrameOffset_SceneTicks = HoldSection->Parameters.StartFrameOffset;
			bSequenceWasModified |= InternalPlay(Item, PlayParams);
		}
		else
		{
			FHoldParams HoldParams;

			const FQualifiedFrameTime GlobalTime = Sequencer->GetGlobalTime();
			UMovieSceneSubSection* CurrentPlayingSection = IsTimeWithinAnySection(GlobalTime, ItemState.WeakPlaySections);
			if (CurrentPlayingSection && IsTimeWithinSection(GlobalTime, CurrentPlayingSection))
			{
				HoldParams.StartFrameOffset_SceneTicks = CurrentPlayingSection->OuterToInnerTransform().TransformTime(GlobalTime.Time);
			}

			bSequenceWasModified |= Stop(Item);
			bSequenceWasModified |= InternalAddHold(Item, HoldParams);
		}
	}
	else
	{
		Item->bHoldAtFirstFrame = !Item->bHoldAtFirstFrame;
	}

	return bSequenceWasModified;
}


bool FSequencerPlaylistItemPlayer_Sequence::InternalStop(USequencerPlaylistItem* Item)
{
	bool bSequenceWasModified = false;

	USequencerPlaylistItem_Sequence* SequenceItem = CastChecked<USequencerPlaylistItem_Sequence>(Item);
	if (!SequenceItem || !SequenceItem->GetSequence())
	{
		return bSequenceWasModified;
	}

	FItemState& ItemState = ItemStates.FindOrAdd(Item);
	UMovieSceneSubSection* HoldSection = ItemState.WeakHoldSection.Get();

	if (ItemState.WeakPlaySections.Num() == 0 && HoldSection == nullptr)
	{
		return bSequenceWasModified;
	}

	if (HoldSection)
	{
		bSequenceWasModified |= EndSection(HoldSection);
		ItemState.WeakHoldSection.Reset();
	}

	for (const TWeakObjectPtr<UMovieSceneSubSection>& WeakPlaySection : ItemState.WeakPlaySections)
	{
		if (UMovieSceneSubSection* PlaySection = WeakPlaySection.Get())
		{
			bSequenceWasModified |= EndSection(PlaySection);
		}
	}

	ItemState.WeakPlaySections.Empty();
	TruncatePlayingUntil(ItemState);

	return bSequenceWasModified;
}


bool FSequencerPlaylistItemPlayer_Sequence::InternalAddHold(USequencerPlaylistItem* Item, const FHoldParams& HoldParams)
{
	USequencerPlaylistItem_Sequence* SequenceItem = CastChecked<USequencerPlaylistItem_Sequence>(Item);
	if (!SequenceItem || !SequenceItem->GetSequence())
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return false;
	}

	FItemState& ItemState = ItemStates.FindOrAdd(Item);
	UMovieSceneSubSection* HoldSection = ItemState.WeakHoldSection.Get();
	if (HoldSection)
	{
		return false;
	}

	ULevelSequence* RootSequence = Cast<ULevelSequence>(Sequencer->GetRootMovieSceneSequence());
	UMovieScene* RootScene = RootSequence->GetMovieScene();

	if (!FSubTrackEditorUtil::CanAddSubSequence(RootSequence, *SequenceItem->GetSequence()))
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("InvalidSequence", "Invalid level sequence {0}. There could be a circular dependency."), SequenceItem->GetSequence()->GetDisplayName()));
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info);
		return false;
	}

	RootSequence->Modify();

	UMovieSceneSubTrack* WorkingTrack = GetOrCreateWorkingTrack(Item);

	ULevelSequence* HoldSequence = SequenceItem->GetSequence();
	UMovieScene* HoldScene = HoldSequence->GetMovieScene();

	const FQualifiedFrameTime GlobalTime = Sequencer->GetGlobalTime();
	const FFrameNumber StartFrame = GlobalTime.Time.FloorToFrame();
	const int32 MaxDuration = TNumericLimits<int32>::Max() - StartFrame.Value - 1;
	HoldSection = WorkingTrack->AddSequence(HoldSequence, StartFrame, MaxDuration);
	ItemState.PlayingUntil_RootTicks = TNumericLimits<int32>::Max();

	const FFrameTime StartFrameOffset_Ticks = HoldParams.StartFrameOffset_SceneTicks.IsSet()
		? HoldParams.StartFrameOffset_SceneTicks.GetValue()
		: ConvertFrameTime(FFrameNumber(SequenceItem->StartFrameOffset),
		                   HoldScene->GetDisplayRate(), HoldScene->GetTickResolution());

	HoldSection->Parameters.StartFrameOffset = StartFrameOffset_Ticks.FloorToFrame();

	HoldSection->Parameters.TimeScale = 0.f;

	ItemState.WeakHoldSection = HoldSection;

	return true;
}


bool FSequencerPlaylistItemPlayer_Sequence::InternalReset(USequencerPlaylistItem* Item)
{
	bool bSequenceWasModified = false;

	if (!Item)
	{
		return bSequenceWasModified;
	}

	bSequenceWasModified |= Stop(Item);

	if (Item->bHoldAtFirstFrame)
	{
		bSequenceWasModified |= AddHold(Item);
	}

	return bSequenceWasModified;
}


bool FSequencerPlaylistItemPlayer_Sequence::IsSequencerRecordingOrPlaying() const
{
	UTakeRecorder* Recorder = UTakeRecorder::GetActiveRecorder();
	if (Recorder && Recorder->GetState() == ETakeRecorderState::Started)
	{
		return true;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return false;
	}

	return Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing;
}


void FSequencerPlaylistItemPlayer_Sequence::TruncatePlayingUntil(FItemState& InItemState)
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		const FFrameNumber Now = Sequencer->GetGlobalTime().Time.FloorToFrame();
		InItemState.PlayingUntil_RootTicks = FMath::Min(Now.Value, InItemState.PlayingUntil_RootTicks);
	}
}


void FSequencerPlaylistItemPlayer_Sequence::ClearItemStates()
{
	ItemStates.Empty();
}


UMovieSceneSubTrack* FSequencerPlaylistItemPlayer_Sequence::GetOrCreateWorkingTrack(USequencerPlaylistItem* Item)
{
	USequencerPlaylistItem_Sequence* SequenceItem = CastChecked<USequencerPlaylistItem_Sequence>(Item);
	if (!SequenceItem || !SequenceItem->GetSequence())
	{
		return nullptr;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return nullptr;
	}

	FItemState& ItemState = ItemStates.FindOrAdd(Item);

	ULevelSequence* RootSequence = Cast<ULevelSequence>(Sequencer->GetRootMovieSceneSequence());
	UMovieScene* RootScene = RootSequence->GetMovieScene();

	// Ensure we have a working track, and that it belongs to the current root sequence.
	if (ItemState.WeakTrack.IsValid() && ItemState.WeakTrack->GetTypedOuter<ULevelSequence>() == RootSequence)
	{
		return ItemState.WeakTrack.Get();
	}

	UMovieSceneSubTrack* NewWorkingTrack = RootScene->AddTrack<UMovieSceneSubTrack>();
	NewWorkingTrack->SetDisplayName(FText::Format(LOCTEXT("SequenceItemTrackName", "Item - {0}"), SequenceItem->GetSequence()->GetDisplayName()));

	// Find or create folder named for our playlist, and organize our track beneath it.
	FText PlaylistName = FText::GetEmpty();
	if (USequencerPlaylist* OuterPlaylist = SequenceItem->GetTypedOuter<USequencerPlaylist>())
	{
		PlaylistName = FText::FromString(*OuterPlaylist->GetName());
	}

	const FText PlaylistFolderNameText = FText::Format(LOCTEXT("PlaylistFolderName", "Playlist - {0}"), PlaylistName);
	const FName PlaylistFolderName = FName(*PlaylistFolderNameText.ToString());

	UMovieSceneFolder* FolderToUse = nullptr;
	for (UMovieSceneFolder* Folder : RootScene->GetRootFolders())
	{
		if (Folder->GetFolderName() == PlaylistFolderName)
		{
			FolderToUse = Folder;
			break;
		}
	}

	if (FolderToUse == nullptr)
	{
		FolderToUse = NewObject<UMovieSceneFolder>(RootScene, NAME_None, RF_Transactional);
		FolderToUse->SetFolderName(PlaylistFolderName);
		RootScene->AddRootFolder(FolderToUse);
	}

	FolderToUse->AddChildTrack(NewWorkingTrack);

	ItemState.WeakTrack = NewWorkingTrack;
	return NewWorkingTrack;
}


bool FSequencerPlaylistItemPlayer_Sequence::EndSection(UMovieSceneSection* Section)
{
	using namespace UE::Private::SequencerPlaylistItem_Sequence;

	bool bSequenceWasModified = false;

	if (!ensure(Section))
	{
		return bSequenceWasModified;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return bSequenceWasModified;
	}

	ULevelSequence* RootSequence = Cast<ULevelSequence>(Sequencer->GetRootMovieSceneSequence());

	UMovieScene* SectionScene = Section->GetTypedOuter<UMovieScene>();
	check(SectionScene);

	const FQualifiedFrameTime GlobalTime = Sequencer->GetGlobalTime();
	const FFrameTime Now_SceneTicks = GlobalTime.ConvertTo(SectionScene->GetTickResolution());
	const FFrameTime SectionStart_SceneTicks = Section->GetInclusiveStartFrame();
	if (SectionStart_SceneTicks > Now_SceneTicks)
	{
		// Remove queued sections entirely (start beyond the current time)
		if (UMovieSceneTrack* SectionTrack = Section->GetTypedOuter<UMovieSceneTrack>())
		{
			RootSequence->Modify();
			bSequenceWasModified = true;

			SectionTrack->Modify();
			SectionTrack->RemoveSection(*Section);
			return bSequenceWasModified;
		}
	}
	else if (IsTimeWithinSection(GlobalTime, Section))
	{
		// Trim sections the current time intersects
		RootSequence->Modify();
		bSequenceWasModified = true;

		Section->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(Now_SceneTicks.FloorToFrame()));

		// Remove degenerate sections. This can happen if we reset then stop a held item while paused.
		if (Section->GetRange().IsEmpty())
		{
			if (UMovieSceneTrack* SectionTrack = Section->GetTypedOuter<UMovieSceneTrack>())
			{
				SectionTrack->Modify();
				SectionTrack->RemoveSection(*Section);
				return bSequenceWasModified;
			}
		}

		// Workaround for not being able to interrupt first playthrough if bCanLoop == true.
		if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			if (SubSection->Parameters.bCanLoop)
			{
				FMovieSceneSubSequenceData SubData(*SubSection);

				TOptional<FFrameTime> FirstLoopEndTime = SubData.OuterToInnerTransform.Inverse().TryTransformTime(SubData.PlayRange.Value.GetUpperBoundValue());
				if (FirstLoopEndTime && GlobalTime.Time <= FirstLoopEndTime.GetValue())
				{
					SubSection->Parameters.bCanLoop = false;
				}
			}
		}
	}

	return bSequenceWasModified;
}


#undef LOCTEXT_NAMESPACE
