// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerPlaylistsModule.h"
#include "Misc/FrameTime.h"
#include "SequencerPlaylistItem.h"
#include "SequencerPlaylistItem_Sequence.generated.h"


class ISequencer;
class ULevelSequence;
class UMovieSceneSection;
class UMovieSceneSubSection;
class UMovieSceneSubTrack;


UCLASS(BlueprintType)
class USequencerPlaylistItem_Sequence : public USequencerPlaylistItem
{
	GENERATED_BODY()

public:
	static FName GetSequencePropertyName();

	//~ Begin USequencerPlaylistItem interface
	virtual FText GetDisplayName() override;
	//~ End USequencerPlaylistItem interface

	//~ Begin UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
	//~ End UObject interface

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	void SetSequence(ULevelSequence* NewSequence);

	ULevelSequence* GetSequence() const { return Sequence; }

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter=SetSequence, Category="Sequencer Playlists", meta=(NoResetToDefault))
	TObjectPtr<ULevelSequence> Sequence;
};


class FSequencerPlaylistItemPlayer_Sequence : public ISequencerPlaylistItemPlayer
{
	struct FItemState
	{
		TWeakObjectPtr<UMovieSceneSubTrack> WeakTrack;
		TWeakObjectPtr<UMovieSceneSubSection> WeakHoldSection;
		TArray<TWeakObjectPtr<UMovieSceneSubSection>> WeakPlaySections;
		int32 PlayingUntil_RootTicks = TNumericLimits<int32>::Min();
		ESequencerPlaylistPlaybackDirection LastPlayDirection = ESequencerPlaylistPlaybackDirection::Forward;
	};

public:
	FSequencerPlaylistItemPlayer_Sequence(TSharedRef<ISequencer> Sequencer);
	virtual ~FSequencerPlaylistItemPlayer_Sequence() override;

	//~ Begin ISequencerPlaylistItemPlayer
	virtual bool Play(USequencerPlaylistItem* Item,
	                  ESequencerPlaylistPlaybackDirection Direction = ESequencerPlaylistPlaybackDirection::Forward) override;
	virtual bool TogglePause(USequencerPlaylistItem* Item) override;
	virtual bool Stop(USequencerPlaylistItem* Item) override;
	virtual bool AddHold(USequencerPlaylistItem* Item) override;
	virtual bool Reset(USequencerPlaylistItem* Item) override;

	virtual FSequencerPlaylistPlaybackState GetPlaybackState(USequencerPlaylistItem* Item) const override;
	//~ End ISequencerPlaylistItemPlayer

private:
	struct FPlayParams
	{
		// If true, playing again before previous plays have completed will
		// queue the additional play at the end.
		// Otherwise, additional plays will occur concurrently on other rows.
		bool bEnqueueExtraPlays = true;

		ESequencerPlaylistPlaybackDirection Direction = ESequencerPlaylistPlaybackDirection::Forward;

		// If not provided, defaults to the Item StartFrameOffset property.
		TOptional<FFrameTime> StartFrameOffset_SceneTicks;

		// If not provided, defaults to the Item EndFrameOffset property.
		TOptional<FFrameTime> EndFrameOffset_SceneTicks;
	};
	bool InternalPlay(USequencerPlaylistItem* Item, const FPlayParams& PlayParams);
	bool InternalPause(USequencerPlaylistItem* Item);
	bool InternalStop(USequencerPlaylistItem* Item);

	struct FHoldParams
	{
		// If not provided, defaults to the Item StartFrameOffset property.
		TOptional<FFrameTime> StartFrameOffset_SceneTicks;
	};
	bool InternalAddHold(USequencerPlaylistItem* Item, const FHoldParams& HoldParams);
	bool InternalReset(USequencerPlaylistItem* Item);

	bool IsSequencerRecordingOrPlaying() const;
	void TruncatePlayingUntil(FItemState& InItemState);
	void ClearItemStates();

	UMovieSceneSubTrack* GetOrCreateWorkingTrack(USequencerPlaylistItem* Item);

	/**
	 * Marks the end of a section based on the item player's Sequencer.
	 * @return true if the current sequence was modified, otherwise false.
	 */
	bool EndSection(UMovieSceneSection* Section);

private:
	TMap<USequencerPlaylistItem*, FItemState> ItemStates;

	TWeakPtr<ISequencer> WeakSequencer;
};
