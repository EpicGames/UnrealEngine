// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerPlaylistPlayer.h"
#include "ISequencer.h"
#include "SequencerPlaylist.h"
#include "SequencerPlaylistsLog.h"
#include "SequencerPlaylistsModule.h"

#include "AnimatedRange.h"
#include "ILevelSequenceEditorToolkit.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "MovieScene.h"
#include "Recorder/TakeRecorder.h"
#include "ScopedTransaction.h"
#include "TakePreset.h"
#include "TakeRecorderSettings.h"

#define LOCTEXT_NAMESPACE "SequencerPlaylists"


USequencerPlaylistPlayer::USequencerPlaylistPlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UTakeRecorder::OnRecordingInitialized().AddUObject(this, &USequencerPlaylistPlayer::OnTakeRecorderInitialized);
		if (UTakeRecorder* ExistingRecorder = UTakeRecorder::GetActiveRecorder())
		{
			OnTakeRecorderInitialized(ExistingRecorder);
		}
	}
}


void USequencerPlaylistPlayer::BeginDestroy()
{
	Super::BeginDestroy();

	PlaylistTicker = {};

	UTakeRecorder::OnRecordingInitialized().RemoveAll(this);

	if (UTakeRecorder* BoundRecorder = WeakRecorder.Get())
	{
		BoundRecorder->OnRecordingStarted().RemoveAll(this);
		BoundRecorder->OnRecordingStopped().RemoveAll(this);
	}
}


void USequencerPlaylistPlayer::SetPlaylist(USequencerPlaylist* InPlaylist)
{
	// Broadcast the event first, so subscribers can still access the previous via GetPlaylist().
	if (InPlaylist != Playlist)
	{
		OnPlaylistSet.Broadcast(this, InPlaylist);
		Playlist = InPlaylist;
	}
}


bool USequencerPlaylistPlayer::PlayItem(
	USequencerPlaylistItem* Item,
	ESequencerPlaylistPlaybackDirection Direction // = ESequencerPlaylistPlaybackDirection::Forward
)
{
	if (!Item)
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();

	FScopedTransaction Transaction(FText::Format(LOCTEXT("PlayItemTransaction", "Begin playback of {0}"), Item->GetDisplayName()));

	EnterUnboundedPlayIfNotRecording();

	if (GetCheckedItemPlayer(Item)->Play(Item, Direction))
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
		return true;
	}
	else
	{
		return false;
	}
}


bool USequencerPlaylistPlayer::PauseItem(USequencerPlaylistItem* Item)
{
	if (!Item)
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();

	FScopedTransaction Transaction(FText::Format(LOCTEXT("PauseItemTransaction", "Toggle pause of {0}"), Item->GetDisplayName()));

	if (GetCheckedItemPlayer(Item)->TogglePause(Item))
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
		return true;
	}
	else
	{
		return false;
	}
}


bool USequencerPlaylistPlayer::StopItem(USequencerPlaylistItem* Item)
{
	if (!Item)
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();

	FScopedTransaction Transaction(FText::Format(LOCTEXT("StopItemTransaction", "Stop playback of {0}"), Item->GetDisplayName()));

	if (GetCheckedItemPlayer(Item)->Stop(Item))
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
		return true;
	}
	else
	{
		return false;
	}
}


bool USequencerPlaylistPlayer::ResetItem(USequencerPlaylistItem* Item)
{
	if (!Item)
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();

	FScopedTransaction Transaction(FText::Format(LOCTEXT("ResetItemTransaction", "Reset playback of {0}"), Item->GetDisplayName()));

	if (GetCheckedItemPlayer(Item)->Reset(Item))
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
		return true;
	}
	else
	{
		return false;
	}
}


bool USequencerPlaylistPlayer::IsPlaying(USequencerPlaylistItem* Item)
{
	return GetPlaybackState(Item).bIsPlaying;
}


FSequencerPlaylistPlaybackState USequencerPlaylistPlayer::GetPlaybackState(USequencerPlaylistItem* Item)
{
	FSequencerPlaylistPlaybackState Result;

	if (!Item)
	{
		return Result;
	}

	// If Sequencer isn't already open, don't open it
	if (WeakSequencer.IsValid())
	{
		Result = GetCheckedItemPlayer(Item)->GetPlaybackState(Item);
	}
	else
	{
		Result.bIsPaused = Item->bHoldAtFirstFrame;
	}

	return Result;
}


namespace UE::Private::PlaylistPlayer
{

TOptional<TRange<double>> ComputeNewRange(TSharedPtr<ISequencer>& Sequencer)
{
	FAnimatedRange Range = Sequencer->GetViewRange();
	UMovieSceneSequence* Sequence  = Sequencer->GetRootMovieSceneSequence();
	if (Sequence)
	{
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (MovieScene)
		{
			FFrameRate FrameRate = MovieScene->GetTickResolution();
			FQualifiedFrameTime GlobalTime = Sequencer->GetGlobalTime();
			FFrameTime CurrentFrameTime = GlobalTime.ConvertTo(FrameRate);
			double CurrentTimeSeconds = FrameRate.AsSeconds(CurrentFrameTime) + 0.5f;
			CurrentTimeSeconds = CurrentTimeSeconds > Range.GetUpperBoundValue() ? CurrentTimeSeconds : Range.GetUpperBoundValue();
			TRange<double> NewRange(Range.GetLowerBoundValue(), CurrentTimeSeconds);
			return NewRange;
		}
	}
	return {};
}

void AdjustMovieSceneRangeForPlay(TSharedPtr<ISequencer>& Sequencer)
{
	check(Sequencer);

	TOptional<TRange<double>> NewRange = ComputeNewRange(Sequencer);
	if (NewRange)
	{
		Sequencer->SetViewRange(*NewRange, EViewRangeInterpolation::Immediate);
		Sequencer->SetClampRange(Sequencer->GetViewRange());
	}
}

FFrameTime GetFrameTime(UMovieScene* MovieScene, FQualifiedFrameTime GlobalTime)
{
	FFrameRate FrameRate = MovieScene->GetTickResolution();
	return GlobalTime.ConvertTo(FrameRate);
}

void SetInfinitePlayRange(TSharedPtr<ISequencer>& Sequencer)
{
	UMovieSceneSequence* Sequence = Sequencer->GetRootMovieSceneSequence();
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	TRange<FFrameNumber> Range = MovieScene->GetPlaybackRange();
	// Set infinite playback range when starting recording. Playback range will be clamped to the bounds of the sections at the completion of the recording
	MovieScene->SetPlaybackRange(TRange<FFrameNumber>(Range.GetLowerBoundValue(), TNumericLimits<int32>::Max() - 1), false);
}

void StopPlaybackAndAdjustTime(TSharedPtr<ISequencer>& Sequencer)
{
	check(Sequencer);

	Sequencer->Pause();
	UMovieSceneSequence* Sequence = Sequencer->GetRootMovieSceneSequence();
	if (Sequence)
	{
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (MovieScene)
		{
			FFrameTime CurrentFrameTime = GetFrameTime(MovieScene, Sequencer->GetGlobalTime());
			TRange<FFrameNumber> Range = MovieScene->GetPlaybackRange();
			// Set the playback range back to a closed interval.
			MovieScene->SetPlaybackRange(TRange<FFrameNumber>(Range.GetLowerBoundValue(), CurrentFrameTime.GetFrame()), false);
		}
	}
}

} // namespace UE::Private::PlaylistPlayer

void USequencerPlaylistPlayer::Tick(float DeltaTime)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	if (Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Paused)
	{
		return;
	}

	if (Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped)
	{
		PlaylistTicker = {};
		UE::Private::PlaylistPlayer::StopPlaybackAndAdjustTime(Sequencer);
		return;
	}

	// Handle a tick
	UE::Private::PlaylistPlayer::AdjustMovieSceneRangeForPlay(Sequencer);
}

void USequencerPlaylistPlayer::EnterUnboundedPlayIfNotRecording()
{
	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();
	UTakeRecorder* Recorder = UTakeRecorder::GetActiveRecorder();

	const bool bInRecorder = Recorder && Recorder->GetState() != ETakeRecorderState::Stopped;
	if (!PlaylistTicker || !bInRecorder)
	{
		PlaylistTicker = MakeUnique<FTickablePlaylist>(this);
	}

	if (PlaylistTicker && Sequencer->GetPlaybackStatus() != EMovieScenePlayerStatus::Playing)
	{
		UE::Private::PlaylistPlayer::SetInfinitePlayRange(Sequencer);
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

		// Tick once to set our playback range.
		Tick(0.0);
	}
}

bool USequencerPlaylistPlayer::PlayAll(
	ESequencerPlaylistPlaybackDirection Direction // = ESequencerPlaylistPlaybackDirection::Forward
)
{
	if (!ensure(Playlist) || !Playlist->Items.Num())
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();

	FScopedTransaction Transaction(LOCTEXT("PlayAllTransaction", "Begin playback of all Playlist items"));

	EnterUnboundedPlayIfNotRecording();

	bool bAnyChange = false;

	for (USequencerPlaylistItem* Item : Playlist->Items)
	{
		if (Item->bMute)
		{
			continue;
		}

		TSharedPtr<ISequencerPlaylistItemPlayer> Player = GetCheckedItemPlayer(Item);
		const FSequencerPlaylistPlaybackState ItemState = Player->GetPlaybackState(Item);

		if (ItemState.bIsPaused)
		{
			bAnyChange |= Player->TogglePause(Item);
		}
		else
		{
			bAnyChange |= Player->Play(Item, Direction);
		}
	}

	if (bAnyChange)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
	}

	return bAnyChange;
}


bool USequencerPlaylistPlayer::PauseAll()
{
	if (!ensure(Playlist) || !Playlist->Items.Num())
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();
	UTakeRecorder* Recorder = UTakeRecorder::GetActiveRecorder();

	const bool bPlaying = Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing;
	const bool bRecording = Recorder && Recorder->GetState() != ETakeRecorderState::Stopped;

	bool bAnyChange = false;

	FScopedTransaction Transaction(LOCTEXT("PauseAllTransaction", "Pause playback of all Playlist items"));

	// If we are playing or recording, pause any playing items.
	// If we are not playing or recording, add holds for any items without holds.
	for (USequencerPlaylistItem* Item : Playlist->Items)
	{
		TSharedPtr<ISequencerPlaylistItemPlayer> Player = GetCheckedItemPlayer(Item);
		const FSequencerPlaylistPlaybackState ItemState = Player->GetPlaybackState(Item);
		if (bPlaying || bRecording)
		{
			if (ItemState.bIsPlaying && !ItemState.bIsPaused)
			{
				bAnyChange |= Player->TogglePause(Item);
			}
		}
		else if (!ItemState.bIsPaused)
		{
			bAnyChange |= Player->AddHold(Item);
		}
	}

	if (bAnyChange)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
	}

	return bAnyChange;
}


bool USequencerPlaylistPlayer::StopAll()
{
	if (!ensure(Playlist) || !Playlist->Items.Num())
	{
		return false;
	}

	bool bAnyChange = false;

	FScopedTransaction Transaction(LOCTEXT("StopAllTransaction", "Stop playback of all Playlist items"));
	for (USequencerPlaylistItem* Item : Playlist->Items)
	{
		bAnyChange |= GetCheckedItemPlayer(Item)->Stop(Item);
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();

	UTakeRecorder* Recorder = UTakeRecorder::GetActiveRecorder();
	const bool bInRecorder = Recorder && Recorder->GetState() != ETakeRecorderState::Stopped;
	if (!bInRecorder && Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing)
	{
		UE::Private::PlaylistPlayer::StopPlaybackAndAdjustTime(Sequencer);
	}

	if (bAnyChange)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
	}

	PlaylistTicker = {};

	return bAnyChange;
}


bool USequencerPlaylistPlayer::ResetAll()
{
	if (!ensure(Playlist) || !Playlist->Items.Num())
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();

	bool bAnyChange = false;

	FScopedTransaction Transaction(LOCTEXT("ResetAllTransaction", "Reset playback of all Playlist items"));
	for (USequencerPlaylistItem* Item : Playlist->Items)
	{
		if (Item->bMute)
		{
			// Stop muted items, but don't add holds for them.
			bAnyChange |= GetCheckedItemPlayer(Item)->Stop(Item);
		}
		else
		{
			bAnyChange |= GetCheckedItemPlayer(Item)->Reset(Item);
		}
	}

	if (bAnyChange)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
	}

	return bAnyChange;
}


TSharedPtr<ISequencer> USequencerPlaylistPlayer::GetOrCreateSequencer()
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return Sequencer;
	}

	ULevelSequence* RootSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	if (!RootSequence)
	{
		UTakePreset* Preset = UTakePreset::AllocateTransientPreset(GetDefault<UTakeRecorderUserSettings>()->LastOpenedPreset.Get());

		if (!Preset->GetLevelSequence())
		{
			FScopedTransaction Transaction(LOCTEXT("CreateEmptyTake", "Create Empty Playlist Sequence"));

			Preset->Modify();
			Preset->CreateLevelSequence();
		}

		RootSequence = Preset->GetLevelSequence();
	}

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RootSequence);
	IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(RootSequence, false);
	ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);

	TSharedPtr<ISequencer> Sequencer = LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
	if (!Sequencer)
	{
		UE_LOGF(LogSequencerPlaylists, Error, "USequencerPlaylistPlayer::GetSequencer: Unable to open Sequencer for asset");
	}
	else
	{
		Sequencer->OnCloseEvent().AddWeakLambda(this, [this](TSharedRef<ISequencer>) {
			// Existing item players invalidated by their sequencer going away.
			ItemPlayersByType.Empty();
		});
	}
	WeakSequencer = Sequencer;
	return Sequencer;
}


void USequencerPlaylistPlayer::OnTakeRecorderInitialized(UTakeRecorder* InRecorder)
{
	if (InRecorder)
	{
		if (UTakeRecorder* PrevRecorder = WeakRecorder.Get())
		{
			PrevRecorder->OnRecordingStarted().RemoveAll(this);
			PrevRecorder->OnRecordingStopped().RemoveAll(this);
		}

		InRecorder->OnRecordingStarted().AddUObject(this, &USequencerPlaylistPlayer::OnTakeRecorderStarted);
		InRecorder->OnRecordingStopped().AddUObject(this, &USequencerPlaylistPlayer::OnTakeRecorderStopped);
		WeakRecorder = InRecorder;
	}
}


void USequencerPlaylistPlayer::OnTakeRecorderStarted(UTakeRecorder* InRecorder)
{
	if (!ensure(Playlist))
	{
		return;
	}

	if (TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer())
	{
		FScopedTransaction Transaction(LOCTEXT("TakeRecorderStartedTransaction", "Playlist - Take Recorder started"));

		for (USequencerPlaylistItem* Item : Playlist->Items)
		{
			if (Item->bHoldAtFirstFrame && !Item->bMute)
			{
				GetCheckedItemPlayer(Item)->AddHold(Item);
			}
		}
	}
}


void USequencerPlaylistPlayer::OnTakeRecorderStopped(UTakeRecorder* InRecorder)
{
	if (!ensure(Playlist))
	{
		return;
	}

	// FIXME: Any sequences not already stopped end up a few frames too long; pass in explicit end frame?
	if (TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer())
	{
		FScopedTransaction Transaction(LOCTEXT("TakeRecorderStoppedTransaction", "Playlist - Take Recorder stopped"));
		bool bAnySequencesModified = false;

		for (USequencerPlaylistItem* Item : Playlist->Items)
		{
			bAnySequencesModified |= GetCheckedItemPlayer(Item)->Stop(Item);
		}

		if (!bAnySequencesModified)
		{
			// Cancel the otherwise empty transaction if stopping the item
			// players did not modify any sequences.
			Transaction.Cancel();
		}
	}
}


TSharedPtr<ISequencerPlaylistItemPlayer> USequencerPlaylistPlayer::GetCheckedItemPlayer(USequencerPlaylistItem* Item)
{
	check(Item);

	TSubclassOf<USequencerPlaylistItem> ItemClass = Item->GetClass();
	if (TSharedRef<ISequencerPlaylistItemPlayer>* ExistingPlayer = ItemPlayersByType.Find(ItemClass))
	{
		return *ExistingPlayer;
	}

	TSharedPtr<ISequencer> Sequencer = GetOrCreateSequencer();
	check(Sequencer.IsValid());

	TSharedPtr<ISequencerPlaylistItemPlayer> NewPlayer =
		static_cast<FSequencerPlaylistsModule&>(FSequencerPlaylistsModule::Get()).CreateItemPlayerForClass(ItemClass, Sequencer.ToSharedRef());
	check(NewPlayer);

	ItemPlayersByType.Add(ItemClass, NewPlayer.ToSharedRef());
	return NewPlayer;
}


#undef LOCTEXT_NAMESPACE
