// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerPlaylistsCommands.h"


#define LOCTEXT_NAMESPACE "SequencerPlaylists"

void FSequencerPlaylistsCommands::RegisterCommands()
{
	UI_COMMAND(OpenPluginWindow, "Sequencer Playlists", "Bring up Sequencer Playlists window", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
