// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "SequencerPlaylistsStyle.h"

class FSequencerPlaylistsCommands : public TCommands<FSequencerPlaylistsCommands>
{
public:

	FSequencerPlaylistsCommands()
		: TCommands<FSequencerPlaylistsCommands>(TEXT("SequencerPlaylists"), NSLOCTEXT("Contexts", "SequencerPlaylists", "Sequencer Playlists Plugin"), NAME_None, FSequencerPlaylistsStyle::GetStyleSetName())
	{
	}

	// TCommands<> interface
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> OpenPluginWindow;
};
