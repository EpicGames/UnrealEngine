// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerPlaylistsModule.h"
#include "AssetTypeActions_SequencerPlaylist.h"
#include "Framework/Commands/UICommandList.h"
#include "SequencerPlaylistItem_Sequence.h"
#include "SequencerPlaylistsLog.h"
#include "SequencerPlaylistsCommands.h"
#include "SequencerPlaylistsSubsystem.h"
#include "SequencerPlaylistsWidgets.h"

#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "Widgets/Docking/SDockTab.h"
#include "ToolMenuSection.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"


const FName FSequencerPlaylistsModule::MainTabType("SequencerPlaylists");
const FName FSequencerPlaylistsModule::AssetEditorTabType("SequencerPlaylistsEditor");


DEFINE_LOG_CATEGORY(LogSequencerPlaylists)


#define LOCTEXT_NAMESPACE "SequencerPlaylists"


void FSequencerPlaylistsModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	RegisterItemPlayer(USequencerPlaylistItem_Sequence::StaticClass(),
		FSequencerPlaylistItemPlayerFactory::CreateLambda(
			[](TSharedRef<ISequencer> Sequencer) {
				return MakeShared<FSequencerPlaylistItemPlayer_Sequence>(Sequencer);
			}
	));

	FSequencerPlaylistsStyle::Initialize();
	FSequencerPlaylistsStyle::ReloadTextures();

	FSequencerPlaylistsCommands::Register();

	PluginCommands = MakeShared<FUICommandList>();

	PluginCommands->MapAction(
		FSequencerPlaylistsCommands::Get().OpenPluginWindow,
		FExecuteAction::CreateRaw(this, &FSequencerPlaylistsModule::PluginButtonClicked),
		FCanExecuteAction());

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FSequencerPlaylistsModule::RegisterMenus));

	const FText TabSpawnerDisplayName = LOCTEXT("TabSpawnerDisplayName", "Playlists");
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MainTabType, FOnSpawnTab::CreateRaw(this, &FSequencerPlaylistsModule::OnSpawnPluginTab))
		.SetDisplayName(TabSpawnerDisplayName)
		.SetTooltipText(LOCTEXT("TabSpawnerTooltipText", "Open the Sequencer Playlists tab."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory())
		.SetIcon(FSlateIcon(FSequencerPlaylistsStyle::GetStyleSetName(), "SequencerPlaylists.TabIcon"));

	// Hidden spawner that permits multiple instances, each spawned without the SaveLayout flag.
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(AssetEditorTabType, FOnSpawnTab::CreateRaw(this, &FSequencerPlaylistsModule::OnSpawnPluginTab))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetDisplayName(TabSpawnerDisplayName)
		.SetIcon(FSlateIcon(FSequencerPlaylistsStyle::GetStyleSetName(), "SequencerPlaylists.TabIcon"))
		.SetReuseTabMethod(FOnFindTabToReuse::CreateLambda([] (const FTabId&)
			{
				return nullptr;
			}
		));

	const FVector2D DefaultPlaylistSize(800.0, 600.0);
	FTabManager::RegisterDefaultTabWindowSize(MainTabType, DefaultPlaylistSize);
	FTabManager::RegisterDefaultTabWindowSize(AssetEditorTabType, DefaultPlaylistSize);

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools")).Get();
	AssetTypeActions = MakeShared<FAssetTypeActions_SequencerPlaylist>();
	AssetTools.RegisterAssetTypeActions(AssetTypeActions.ToSharedRef());
}


void FSequencerPlaylistsModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	UToolMenus::UnRegisterStartupCallback(this);

	UToolMenus::UnregisterOwner(this);

	FSequencerPlaylistsStyle::Shutdown();

	FSequencerPlaylistsCommands::Unregister();

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MainTabType);
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(AssetEditorTabType);

	FTabManager::UnregisterDefaultTabWindowSize(MainTabType);
	FTabManager::UnregisterDefaultTabWindowSize(AssetEditorTabType);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetTools")))
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.UnregisterAssetTypeActions(AssetTypeActions.ToSharedRef());
	}
}


bool FSequencerPlaylistsModule::RegisterItemPlayer(TSubclassOf<USequencerPlaylistItem> ItemClass, FSequencerPlaylistItemPlayerFactory PlayerFactory)
{
	if (ItemPlayerFactories.Contains(ItemClass))
	{
		return false;
	}

	ItemPlayerFactories.Add(ItemClass, PlayerFactory);
	return true;
}


TSharedPtr<ISequencerPlaylistItemPlayer> FSequencerPlaylistsModule::CreateItemPlayerForClass(TSubclassOf<USequencerPlaylistItem> ItemClass, TSharedRef<ISequencer> Sequencer)
{
	if (FSequencerPlaylistItemPlayerFactory* Factory = ItemPlayerFactories.Find(ItemClass))
	{
		return Factory->Execute(Sequencer);
	}

	return nullptr;
}


TSharedRef<SDockTab> FSequencerPlaylistsModule::OnSpawnPluginTab(const FSpawnTabArgs& SpawnTabArgs)
{
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	TSharedRef<SSequencerPlaylistPanel> Panel = SNew(SSequencerPlaylistPanel, DockTab);

	USequencerPlaylistsSubsystem* Subsystem = GEditor->GetEditorSubsystem<USequencerPlaylistsSubsystem>();
	USequencerPlaylistPlayer* Player = Subsystem->CreatePlayerForEditor(Panel);
	Panel->SetPlayer(Player);

	DockTab->SetContent(Panel);
	DockTab->SetLabel(TAttribute<FText>::CreateLambda(
		[
			WeakPanel = TWeakPtr<SSequencerPlaylistPanel>(Panel)
		]() -> FText
		{
			if (TSharedPtr<SSequencerPlaylistPanel> Panel = WeakPanel.Pin())
			{
				return FText::AsCultureInvariant(Panel->GetDisplayTitle());
			}

			return FText::GetEmpty();
		}
	));

	return DockTab;
}


TSharedRef<SDockTab> FSequencerPlaylistsModule::SpawnEditorForPlaylist(USequencerPlaylist* Playlist)
{
	// ETabIdFlags::SaveLayout flag is unset for tabs other than the one spawned via the menu.
	FTabId TabId = FTabId(AssetEditorTabType, ETabIdFlags::None);

	TSharedPtr<SDockTab> PlaylistTab = FGlobalTabmanager::Get()->TryInvokeTab(TabId);
	check(PlaylistTab);

	TSharedRef<SSequencerPlaylistPanel> Panel = StaticCastSharedRef<SSequencerPlaylistPanel>(PlaylistTab->GetContent());
	Panel->LoadPlaylist(Playlist);

	return PlaylistTab.ToSharedRef();
}


void FSequencerPlaylistsModule::PluginButtonClicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(MainTabType);
}


void FSequencerPlaylistsModule::RegisterMenus()
{
	// Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	FToolMenuOwnerScoped OwnerScoped(this);

	{
		UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar");
		{
			FToolMenuSection& Section = ToolbarMenu->FindOrAddSection("Settings");
			{
				FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(FSequencerPlaylistsCommands::Get().OpenPluginWindow));
				Entry.SetCommandList(PluginCommands);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE(FSequencerPlaylistsModule, SequencerPlaylists)
