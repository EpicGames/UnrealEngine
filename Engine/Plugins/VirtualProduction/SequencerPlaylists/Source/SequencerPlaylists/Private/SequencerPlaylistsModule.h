// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "ISequencerPlaylistsModule.h"


class FAssetTypeActions_SequencerPlaylist;
class FUICommandList;
class SDockTab;
class USequencerPlaylist;


class FSequencerPlaylistsModule : public ISequencerPlaylistsModule
{
public:
	static const FName MainTabType;
	static const FName AssetEditorTabType;

	//~ Begin IModuleInterface
	void StartupModule() override;
	void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin ISequencerPlaylistsModule
	bool RegisterItemPlayer(TSubclassOf<USequencerPlaylistItem> ItemClass, FSequencerPlaylistItemPlayerFactory PlayerFactory) override;
	//~ End ISequencerPlaylistsModule

	TSharedPtr<ISequencerPlaylistItemPlayer> CreateItemPlayerForClass(TSubclassOf<USequencerPlaylistItem> ItemClass, TSharedRef<ISequencer> Sequencer);

	TSharedPtr<FUICommandList> GetCommandList() { return PluginCommands; }

	TSharedRef<SDockTab> SpawnEditorForPlaylist(USequencerPlaylist* Playlist);

private:
	/** This function will be bound to Command (by default it will bring up plugin window) */
	void PluginButtonClicked();

	void RegisterMenus();

	TSharedRef<SDockTab> OnSpawnPluginTab(const class FSpawnTabArgs& SpawnTabArgs);

private:
	TMap<TSubclassOf<USequencerPlaylistItem>, FSequencerPlaylistItemPlayerFactory> ItemPlayerFactories;

	TSharedPtr<FUICommandList> PluginCommands;

	TSharedPtr<FAssetTypeActions_SequencerPlaylist> AssetTypeActions;
};
