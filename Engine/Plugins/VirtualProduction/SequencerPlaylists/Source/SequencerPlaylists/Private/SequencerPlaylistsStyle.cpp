// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerPlaylistsStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/SlateStyleMacros.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/Paths.h"
#include "Styling/StyleColors.h"
#include "Misc/ScopeExit.h"
#include "Styling/ToolBarStyle.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"


#define RootToContentDir Style->RootToContentDir
#define RootToCoreContentDir Style->RootToCoreContentDir


TSharedPtr<FSlateStyleSet> FSequencerPlaylistsStyle::StyleInstance = nullptr;


void FSequencerPlaylistsStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FSequencerPlaylistsStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FSequencerPlaylistsStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("SequencerPlaylistsStyle"));
	return StyleSetName;
}

FLinearColor FSequencerPlaylistsStyle::MakeColorVariation(FLinearColor InColor, EColorVariation Variation)
{
	const FLinearColor HsvColor = InColor.LinearRGBToHSV();
	switch (Variation)
	{
		case EColorVariation::Desaturated: return FLinearColor(HsvColor.R, HsvColor.G * .5, HsvColor.B, HsvColor.A).HSVToLinearRGB();
		case EColorVariation::Dimmed: return FLinearColor(HsvColor.R, HsvColor.G, HsvColor.B * .5, HsvColor.A).HSVToLinearRGB();
		default: checkNoEntry();
	}

	return InColor;
};

TSharedRef<FSlateStyleSet> FSequencerPlaylistsStyle::Create()
{
	static const FVector2D Icon12x12(12.0f, 12.0f);
	static const FVector2D Icon16x16(16.0f, 16.0f);
	static const FVector2D Icon20x20(20.0f, 20.0f);
	static const FVector2D Icon24x24(24.0f, 24.0f);

	const FString PluginContentRoot = IPluginManager::Get().FindPlugin("SequencerPlaylists")->GetBaseDir() / TEXT("Resources");
	const FString EngineSlateDir = FPaths::EngineContentDir() / TEXT("Slate");
	const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor") / TEXT("Slate");

	TSharedRef<FSlateStyleSet> Style = MakeShareable(new FSlateStyleSet("SequencerPlaylistsStyle"));
	Style->SetContentRoot(PluginContentRoot);

	Style->Set("SequencerPlaylists.TabIcon", new IMAGE_BRUSH_SVG("Playlist", Icon16x16));
	Style->Set("SequencerPlaylists.Panel.Background", new FSlateColorBrush(FStyleColors::Panel));

	FLinearColor DimColor(FSlateColor(EStyleColor::Background).GetSpecifiedColor());
	DimColor.A = 0.5f;
	Style->Set("SequencerPlaylists.Item.Dim", new FSlateColorBrush(DimColor));

	Style->Set("SequencerPlaylists.NewPlaylist", new IMAGE_BRUSH_SVG("PlaylistNew", Icon20x20));
	// SavePlaylist is registered under a different root below.
	Style->Set("SequencerPlaylists.OpenPlaylist", new IMAGE_BRUSH_SVG("PlaylistLoad", Icon20x20));

	Style->Set("SequencerPlaylists.PlayMode", new IMAGE_BRUSH_SVG("PlaybackAuto", Icon20x20));

	Style->Set("SequencerPlaylists.HoldFrame", new IMAGE_BRUSH_SVG("HoldFrame", Icon16x16));

	Style->Set("SequencerPlaylists.Loop.Disabled", new IMAGE_BRUSH_SVG("LoopNot", Icon16x16));
	Style->Set("SequencerPlaylists.Loop.Finite", new IMAGE_BRUSH_SVG("LoopNumber", Icon16x16));

	// Play, Stop, Reset are registered under a different root below.
	Style->Set("SequencerPlaylists.Pause", new IMAGE_BRUSH_SVG("Pause", Icon20x20));
	Style->Set("SequencerPlaylists.Pause.Small", new IMAGE_BRUSH_SVG("Pause", Icon16x16));
	Style->Set("SequencerPlaylists.PlayReverse", new IMAGE_BRUSH_SVG("PlayReverse", Icon20x20));
	Style->Set("SequencerPlaylists.PlayReverse.Small", new IMAGE_BRUSH_SVG("PlayReverse", Icon16x16));

	// Main toolbar
	{
		const FToolBarStyle& SlimToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
		FToolBarStyle MainToolbarStyle = FToolBarStyle(SlimToolbarStyle)
			.SetIconSize(Icon20x20)
			.SetComboButtonPadding(FMargin(2.0f, 0.0f));

		FButtonStyle MainToolbarButtonStyle = FButtonStyle(SlimToolbarStyle.ButtonStyle)
			.SetNormalPadding(FMargin(4.f, 4.f, 4.f, 4.f))
			.SetPressedPadding(FMargin(4.f, 5.f, 4.f, 3.f));
		MainToolbarStyle.SetButtonStyle(MainToolbarButtonStyle);

		FComboButtonStyle MainToolbarComboButtonStyle = FComboButtonStyle(SlimToolbarStyle.ComboButtonStyle)
			.SetDownArrowPadding(FMargin(1.f, 0.f, 0.f, 0.f))
			.SetButtonStyle(MainToolbarButtonStyle);
		MainToolbarStyle.SetComboButtonStyle(MainToolbarComboButtonStyle);

		{
			// Borrowed asset; change the root temporarily
			Style->SetContentRoot(EngineSlateDir);
			ON_SCOPE_EXIT{ Style->SetContentRoot(PluginContentRoot); };

			FComboButtonStyle MainToolbarSettingsComboButtonStyle = FComboButtonStyle(SlimToolbarStyle.SettingsComboButton)
				.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2D(4, 16)));
			MainToolbarStyle.SetSettingsComboButtonStyle(MainToolbarSettingsComboButtonStyle);
		}

		FCheckBoxStyle MainToolbarToggleButtonStyle = FCheckBoxStyle(SlimToolbarStyle.ToggleButton)
			.SetPadding(FMargin(4.f, 4.f, 4.f, 4.f));
		MainToolbarStyle.SetToggleButtonStyle(MainToolbarToggleButtonStyle);

		Style->Set("SequencerPlaylists.MainToolbar", MainToolbarStyle);
	}

	const FLinearColor PlayColor = FStyleColors::AccentGreen.GetSpecifiedColor();
	const FLinearColor StopColor = FStyleColors::AccentRed.GetSpecifiedColor();
	const FLinearColor PauseResetColor = FStyleColors::AccentBlue.GetSpecifiedColor();
	const FLinearColor PlayPressedColor = MakeColorVariation(PlayColor, EColorVariation::Dimmed);
	const FLinearColor StopPressedColor = MakeColorVariation(StopColor, EColorVariation::Dimmed);
	const FLinearColor PauseResetPressedColor = MakeColorVariation(PauseResetColor, EColorVariation::Dimmed);

	const float CornerRadius = 4.0f;
	const FVector4 AllCorners(CornerRadius, CornerRadius, CornerRadius, CornerRadius);
	const FVector4 LeftCorners(CornerRadius, 0.0f, 0.0f, CornerRadius);
	const FVector4 MiddleCorners(0.0f, 0.0f, 0.0f, 0.0f);
	const FVector4 RightCorners(0.0f, CornerRadius, CornerRadius, 0.0f);

	// Table rows
	{
		const FTableRowStyle& NormalTableRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
		FTableRowStyle PlaylistItemRowStyle = FTableRowStyle(NormalTableRowStyle)
			.SetEvenRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(FSlateNoResource());
		Style->Set("SequencerPlaylists.ItemRow", PlaylistItemRowStyle);

		Style->Set("SequencerPlaylists.ItemRow.BackgroundOuter", new FSlateColorBrush(FStyleColors::Recessed));
		Style->Set("SequencerPlaylists.ItemRow.BackgroundInner", new FSlateRoundedBoxBrush(FStyleColors::Panel, AllCorners));
	}


	const FSlateRoundedBoxBrush LeftBrush(FStyleColors::Dropdown, LeftCorners);
	const FSlateRoundedBoxBrush MiddleBrush(FStyleColors::Dropdown, MiddleCorners);
	const FSlateRoundedBoxBrush RightBrush(FStyleColors::Dropdown, RightCorners);

	FButtonStyle TransportButtonStyle = FButtonStyle()
		.SetNormalForeground(FStyleColors::Foreground)
		.SetDisabledForeground(FStyleColors::Foreground);

	const FButtonStyle LeftTransportButton = FButtonStyle(TransportButtonStyle)
		.SetNormal(LeftBrush)
		.SetHovered(LeftBrush)
		.SetPressed(LeftBrush)
		.SetDisabled(LeftBrush)
		.SetNormalPadding(FMargin(8.f, 4.f, 6.f, 4.f))
		.SetPressedPadding(FMargin(8.f, 4.f, 6.f, 4.f));

	const FButtonStyle MiddleTransportButton = FButtonStyle(TransportButtonStyle)
		.SetNormal(MiddleBrush)
		.SetHovered(MiddleBrush)
		.SetPressed(MiddleBrush)
		.SetDisabled(MiddleBrush)
		.SetNormalPadding(FMargin(6.f, 4.f, 6.f, 4.f))
		.SetPressedPadding(FMargin(6.f, 4.f, 6.f, 4.f));

	const FButtonStyle RightTransportButton = FButtonStyle(TransportButtonStyle)
		.SetNormal(RightBrush)
		.SetHovered(RightBrush)
		.SetPressed(RightBrush)
		.SetDisabled(RightBrush)
		.SetNormalPadding(FMargin(6.f, 4.f, 8.f, 4.f))
		.SetPressedPadding(FMargin(6.f, 4.f, 8.f, 4.f));

	const FButtonStyle PlayReverseTransportButton = FButtonStyle(LeftTransportButton)
		.SetHoveredForeground(PlayColor)
		.SetPressedForeground(PlayPressedColor);

	const FButtonStyle PauseTransportButton = FButtonStyle(MiddleTransportButton)
		.SetHoveredForeground(PauseResetColor)
		.SetPressedForeground(PauseResetPressedColor);

	const FButtonStyle PlayTransportButton = FButtonStyle(MiddleTransportButton)
		.SetHoveredForeground(PlayColor)
		.SetPressedForeground(PlayPressedColor);

	const FButtonStyle StopTransportButton = FButtonStyle(MiddleTransportButton)
		.SetHoveredForeground(StopColor)
		.SetPressedForeground(StopPressedColor);

	const FButtonStyle ResetTransportButton = FButtonStyle(RightTransportButton)
		.SetHoveredForeground(PauseResetColor)
		.SetPressedForeground(PauseResetPressedColor);

	Style->Set("SequencerPlaylists.TransportButton.PlayReverse", PlayReverseTransportButton);
	Style->Set("SequencerPlaylists.TransportButton.Pause", PauseTransportButton);
	Style->Set("SequencerPlaylists.TransportButton.Play", PlayTransportButton);
	Style->Set("SequencerPlaylists.TransportButton.Stop", StopTransportButton);
	Style->Set("SequencerPlaylists.TransportButton.Reset", ResetTransportButton);

	const FButtonStyle HoverTransportButtonStyle = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetNormalForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(0, 0, 0, 0))
		.SetPressedPadding(FMargin(0, 0, 0, 0));

	const FButtonStyle PlayHoverTransportButton = FButtonStyle(HoverTransportButtonStyle)
		.SetHoveredForeground(PlayColor)
		.SetPressedForeground(PlayPressedColor);

	const FButtonStyle PauseHoverTransportButton = FButtonStyle(HoverTransportButtonStyle)
		.SetHoveredForeground(PauseResetColor)
		.SetPressedForeground(PauseResetPressedColor);

	const FButtonStyle StopHoverTransportButton = FButtonStyle(HoverTransportButtonStyle)
		.SetHoveredForeground(StopColor)
		.SetPressedForeground(StopPressedColor);

	const FButtonStyle ResetHoverTransportButton = FButtonStyle(HoverTransportButtonStyle)
		.SetHoveredForeground(PauseResetColor)
		.SetPressedForeground(PauseResetPressedColor);

	const FButtonStyle PlayReverseHoverTransportButton = FButtonStyle(HoverTransportButtonStyle)
		.SetHoveredForeground(PlayColor)
		.SetPressedForeground(PlayPressedColor);

	Style->Set("SequencerPlaylists.HoverTransport.Play", PlayHoverTransportButton);
	Style->Set("SequencerPlaylists.HoverTransport.Pause", PauseHoverTransportButton);
	Style->Set("SequencerPlaylists.HoverTransport.Stop", StopHoverTransportButton);
	Style->Set("SequencerPlaylists.HoverTransport.Reset", ResetHoverTransportButton);
	Style->Set("SequencerPlaylists.HoverTransport.PlayReverse", PlayReverseHoverTransportButton);

	FEditableTextBoxStyle EditableTextStyle = FEditableTextBoxStyle()
		.SetTextStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 9))
		.SetBackgroundImageNormal(FSlateNoResource())
		.SetBackgroundImageHovered(FSlateNoResource())
		.SetBackgroundImageFocused(FSlateNoResource())
		.SetBackgroundImageReadOnly(FSlateNoResource())
		.SetBackgroundColor(FLinearColor::Transparent)
		.SetForegroundColor(FSlateColor::UseForeground());

	Style->Set("SequencerPlaylists.EditableTextBox", EditableTextStyle);
	Style->Set("SequencerPlaylists.TitleFont", FCoreStyle::GetDefaultFontStyle("Regular", 13));
	Style->Set("SequencerPlaylists.DescriptionFont", FCoreStyle::GetDefaultFontStyle("Regular", 8));

	{
		// These assets live in engine slate folders; change the root temporarily
		ON_SCOPE_EXIT{ Style->SetContentRoot(PluginContentRoot); };

		// Engine/Content/Slate/...
		{
			Style->SetContentRoot(EngineSlateDir);
			Style->Set("SequencerPlaylists.Ellipsis", new IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2D(6, 24)));
		}

		// Engine/Content/Editor/Slate/...
		{
			Style->SetContentRoot(EngineEditorSlateDir);
			Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

			Style->Set("SequencerPlaylists.SavePlaylist", new IMAGE_BRUSH_SVG("Starship/Common/SaveCurrent", Icon20x20));

			Style->Set("SequencerPlaylists.Play", new IMAGE_BRUSH_SVG("Starship/Common/play", Icon20x20));
			Style->Set("SequencerPlaylists.Play.Small", new IMAGE_BRUSH_SVG("Starship/Common/play", Icon16x16));
			Style->Set("SequencerPlaylists.Stop", new IMAGE_BRUSH_SVG("Starship/Common/stop", Icon20x20));
			Style->Set("SequencerPlaylists.Stop.Small", new IMAGE_BRUSH_SVG("Starship/Common/stop", Icon16x16));
			Style->Set("SequencerPlaylists.Reset", new IMAGE_BRUSH_SVG("Starship/Common/Reset", Icon20x20));
			Style->Set("SequencerPlaylists.Reset.Small", new IMAGE_BRUSH_SVG("Starship/Common/Reset", Icon16x16));
		}
	}

	return Style;
}

void FSequencerPlaylistsStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FSequencerPlaylistsStyle::Get()
{
	return *StyleInstance;
}
