// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class ISlateStyle;

/**  */
class FSequencerPlaylistsStyle
{
public:
	static void Initialize();
	static void Shutdown();

	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set */
	static const ISlateStyle& Get();

	static FName GetStyleSetName();

private:
	enum class EColorVariation
	{
		Desaturated,
		Dimmed
	};

	static FLinearColor MakeColorVariation(FLinearColor InColor, EColorVariation Variation);

	static TSharedRef< class FSlateStyleSet > Create();

private:
	static TSharedPtr< class FSlateStyleSet > StyleInstance;
};
