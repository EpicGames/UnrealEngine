// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerPlaylistsSubsystem.h"
#include "ConcertSequencerMessages.h"
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"
#include "IConcertSession.h"
#include "LevelSequence.h"
#include "Logging/StructuredLog.h"
#include "SequencerPlaylist.h"
#include "SequencerPlaylistItem_Sequence.h"
#include "SequencerPlaylistPlayer.h"
#include "SequencerPlaylistsLog.h"
#include "UObject/Package.h"


#define LOCTEXT_NAMESPACE "SequencerPlaylists"


void USequencerPlaylistsSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
	{
		IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
		ConcertClient->OnSessionStartup().AddUObject(this, &USequencerPlaylistsSubsystem::OnConcertSessionStartup);
		ConcertClient->OnSessionShutdown().AddUObject(this, &USequencerPlaylistsSubsystem::OnConcertSessionShutdown);

		if (TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
		{
			OnConcertSessionStartup(ConcertClientSession.ToSharedRef());
		}
	}
}


void USequencerPlaylistsSubsystem::Deinitialize()
{
	Super::Deinitialize();

	ensureMsgf(EditorPlaylists.Num() == 0, TEXT("USequencerPlaylistsSubsystem::EditorPlaylists leak"));
	ensureMsgf(EditorPlayers.Num() == 0, TEXT("USequencerPlaylistsSubsystem::EditorPlayers leak"));
	ensureMsgf(EditorPackages.Num() == 0, TEXT("USequencerPlaylistsSubsystem::EditorPackages leak"));
}


USequencerPlaylistPlayer* USequencerPlaylistsSubsystem::CreatePlayerForEditor(TSharedRef<SSequencerPlaylistPanel> Editor)
{
	FSequencerPlaylistEditorHandle EditorHandle(&Editor.Get());

	TObjectPtr<USequencerPlaylistPlayer>* ExistingPlayer = EditorPlayers.Find(EditorHandle);
	if (!ensure(ExistingPlayer == nullptr))
	{
		return *ExistingPlayer;
	}

	ensure(EditorPackages.Find(EditorHandle) == nullptr);
	ensure(EditorPlaylists.Find(EditorHandle) == nullptr);

	USequencerPlaylist* NewPlaylist = CreateTransientPlaylistForEditor(Editor);
	EditorPlaylists.Add(EditorHandle, NewPlaylist);
	EditorPackages.Add(EditorHandle, NewPlaylist->GetPackage());

	USequencerPlaylistPlayer* NewPlayer = NewObject<USequencerPlaylistPlayer>();
	EditorPlayers.Add(EditorHandle, NewPlayer);
	NewPlayer->SetPlaylist(NewPlaylist);

	return NewPlayer;
}


USequencerPlaylist* USequencerPlaylistsSubsystem::CreateTransientPlaylistForEditor(TSharedRef<SSequencerPlaylistPanel> Editor)
{
	FName PackageName = ::MakeUniqueObjectName(GetTransientPackage(), UPackage::StaticClass(), TEXT("SequencerPlaylist"));
	UPackage* PlaylistPackage = NewObject<UPackage>(nullptr, PackageName, RF_Transient | RF_Transactional);
	return NewObject<USequencerPlaylist>(PlaylistPackage, TEXT("UntitledPlaylist"), RF_Transactional);
}


void USequencerPlaylistsSubsystem::NotifyEditorClosed(SSequencerPlaylistPanel* Editor)
{
	check(Editor);

	FSequencerPlaylistEditorHandle EditorHandle(Editor);

	ensure(EditorPlaylists.Remove(EditorHandle));
	ensure(EditorPlayers.Remove(EditorHandle));
	ensure(EditorPackages.Remove(EditorHandle));

	UpdatePreloadSet();
}


bool USequencerPlaylistsSubsystem::IsPreloadingActive() const
{
	// If we're in a multi-user session, we are responding to remote preloading requests.
	return WeakSession.IsValid();
}


TPair<EConcertSequencerPreloadStatus, FText> USequencerPlaylistsSubsystem::GetPreloadStatusForSequence(const FTopLevelAssetPath& SequencePath)
{
	TPair<EConcertSequencerPreloadStatus, FText> Result;

	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
	if (!ensure(Session))
	{
		Result.Key = EConcertSequencerPreloadStatus::Failed;
		Result.Value = LOCTEXT("PreloadStatusNoSession", "Not in multi-user session");
		return Result;
	}

	const TArray<FConcertSessionClientInfo> AllClients = Session->GetSessionClients();
	TSet<FGuid> PendingClients, FailedClients;
	TMap<FGuid, FConcertClientInfo> ClientGuidToInfo;
	for (const FConcertSessionClientInfo& Client : AllClients)
	{
		ClientGuidToInfo.Add(Client.ClientEndpointId, Client.ClientInfo);

		const FClientSequenceKey CompositeKey{ Client.ClientEndpointId, SequencePath };
		EConcertSequencerPreloadStatus Status = EConcertSequencerPreloadStatus::Pending;
		if (const EConcertSequencerPreloadStatus* MaybeStatus = PreloadStates.Find(CompositeKey))
		{
			Status = *MaybeStatus;
		}

		switch (Status)
		{
		case EConcertSequencerPreloadStatus::Pending:
			PendingClients.Add(Client.ClientEndpointId);
			break;
		case EConcertSequencerPreloadStatus::Failed:
			FailedClients.Add(Client.ClientEndpointId);
			break;
		}
	}

	// Determine overall status enum
	if (FailedClients.Num() > 0)
	{
		Result.Key = EConcertSequencerPreloadStatus::Failed;
	}
	else if (PendingClients.Num() > 0)
	{
		Result.Key = EConcertSequencerPreloadStatus::Pending;
	}
	else
	{
		Result.Key = EConcertSequencerPreloadStatus::Succeeded;
	}

	// Construct accompanying textual detail
	const FText PreloadClientDelimiter = LOCTEXT("PreloadStatusClientDelimiter", ", ");
	TArray<FText> StatusGroups;
	if (FailedClients.Num())
	{
		TArray<FText> ClientDisplayNames;
		for (const FGuid& ClientId : FailedClients)
		{
			ClientDisplayNames.Add(FText::FromString(ClientGuidToInfo[ClientId].DisplayName));
		}
		StatusGroups.Add(FText::Format(
			LOCTEXT("PreloadStatusFailedList", "Failed: {0}"),
			FText::Join(PreloadClientDelimiter, ClientDisplayNames)));
	}

	if (PendingClients.Num())
	{
		TArray<FText> ClientDisplayNames;
		for (const FGuid& ClientId : PendingClients)
		{
			ClientDisplayNames.Add(FText::FromString(ClientGuidToInfo[ClientId].DisplayName));
		}
		StatusGroups.Add(FText::Format(
			LOCTEXT("PreloadStatusPendingList", "Pending: {0}"),
			FText::Join(PreloadClientDelimiter, ClientDisplayNames)));
	}

	if (FailedClients.Num() == 0 && PendingClients.Num() == 0)
	{
		StatusGroups.Add(LOCTEXT("PreloadStatusAllSuccessful", "All clients preloaded successfully."));
	}
	else
	{
		TArray<FText> ClientDisplayNames;
		for (const FConcertSessionClientInfo& Client : AllClients)
		{
			if (!FailedClients.Contains(Client.ClientEndpointId) && !PendingClients.Contains(Client.ClientEndpointId))
			{
				ClientDisplayNames.Add(FText::FromString(Client.ClientInfo.DisplayName));
			}
		}

		if (ClientDisplayNames.Num())
		{
			StatusGroups.Add(FText::Format(
				LOCTEXT("PreloadStatusCompleteList", "Complete: {0}"),
				FText::Join(PreloadClientDelimiter, ClientDisplayNames)));
		}
	}

	Result.Value = FText::Join(LOCTEXT("PreloadGroupsDelimiter", "\n"), StatusGroups);
	return Result;
}


void USequencerPlaylistsSubsystem::UpdatePreloadSet()
{
	// Determine which sequences (if any) were added to/removed from the preload set.
	// We copy the previous set, recompute the current set, then compare them.
	const TSet<TObjectPtr<ULevelSequence>> PreviousPreload = PreloadSequences;

	PreloadSequences.Empty(PreloadSequences.Num());

	using FPlaylistPair = TPair<FSequencerPlaylistEditorHandle, TObjectPtr<USequencerPlaylist>>;
	for (const FPlaylistPair& EditorPlaylist : EditorPlaylists)
	{
		for (USequencerPlaylistItem* Item : EditorPlaylist.Value->Items)
		{
			USequencerPlaylistItem_Sequence* SequenceItem = Cast<USequencerPlaylistItem_Sequence>(Item);
			if (SequenceItem && SequenceItem->GetSequence())
			{
				PreloadSequences.Add(SequenceItem->GetSequence());
			}
		}
	}

	FConcertSequencerPreloadRequest PreloadAddedEvent{ .bShouldBePreloaded = true };
	FConcertSequencerPreloadRequest PreloadRemovedEvent{ .bShouldBePreloaded = false };

	const TSet<TObjectPtr<ULevelSequence>> PreloadUnion = PreviousPreload.Union(PreloadSequences);
	for (const TObjectPtr<ULevelSequence>& Sequence : PreloadUnion)
	{
		const FTopLevelAssetPath SequenceObjectPath(Sequence);
		if (!PreviousPreload.Contains(Sequence))
		{
			UE_LOGFMT(LogSequencerPlaylists, Verbose, "Adding sequence '{Asset}' to preload set", UE::FAssetLog(Sequence));
			PreloadAddedEvent.SequenceObjectPaths.Add(SequenceObjectPath);
		}
		else if (!PreloadSequences.Contains(Sequence))
		{
			UE_LOGFMT(LogSequencerPlaylists, Verbose, "Removing sequence '{Asset}' from preload set", UE::FAssetLog(Sequence));
			PreloadRemovedEvent.SequenceObjectPaths.Add(SequenceObjectPath);
		}
	}

	if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
	{
		if (PreloadAddedEvent.SequenceObjectPaths.Num() > 0)
		{
			Session->SendCustomEvent(PreloadAddedEvent, Session->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		}

		if (PreloadRemovedEvent.SequenceObjectPaths.Num() > 0)
		{
			Session->SendCustomEvent(PreloadRemovedEvent, Session->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		}
	}
}


void USequencerPlaylistsSubsystem::OnConcertSessionStartup(TSharedRef<IConcertClientSession> InSession)
{
	WeakSession = InSession;

	InSession->OnSessionClientChanged().AddUObject(this, &USequencerPlaylistsSubsystem::HandleSessionClientChanged);
	InSession->RegisterCustomEventHandler<FConcertSequencerPreloadClientStatusMap>(this, &USequencerPlaylistsSubsystem::HandleSequencerPreloadStatusEvent);

	// Notify the server of any sequences we already wanted preloaded prior to joining the session.
	FConcertSequencerPreloadRequest PreloadEvent{ .bShouldBePreloaded = true };

	for (const TObjectPtr<ULevelSequence>& Sequence : PreloadSequences)
	{
		PreloadEvent.SequenceObjectPaths.Add(FTopLevelAssetPath(Sequence));
	}

	if (PreloadEvent.SequenceObjectPaths.Num() > 0)
	{
		InSession->SendCustomEvent(PreloadEvent, InSession->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
	}
}


void USequencerPlaylistsSubsystem::OnConcertSessionShutdown(TSharedRef<IConcertClientSession> InSession)
{
	ensure(WeakSession == InSession);
	WeakSession = nullptr;
}


void USequencerPlaylistsSubsystem::HandleSessionClientChanged(IConcertClientSession& InSession, EConcertClientStatus InClientStatus, const FConcertSessionClientInfo& InClientInfo)
{
	// Clear any PreloadStates belonging to a disconnecting client.
	if (InClientStatus == EConcertClientStatus::Disconnected)
	{
		const FGuid& DisconnectingClient = InClientInfo.ClientEndpointId;
		for (TMap<FClientSequenceKey, EConcertSequencerPreloadStatus>::TIterator Iter = PreloadStates.CreateIterator(); Iter; ++Iter)
		{
			const FClientSequenceKey& ClientSeqPair = Iter->Key;
			if (ClientSeqPair.Key == DisconnectingClient)
			{
				Iter.RemoveCurrent();
			}
		}
	}
}


void USequencerPlaylistsSubsystem::HandleSequencerPreloadStatusEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerPreloadClientStatusMap& InEvent)
{
	for (const TPair<FGuid, FConcertSequencerPreloadAssetStatusMap>& EndpointStatus : InEvent.ClientEndpoints)
	{
		const FGuid& Endpoint = EndpointStatus.Key;
		for (const TPair<FTopLevelAssetPath, EConcertSequencerPreloadStatus>& Status : EndpointStatus.Value.Sequences)
		{
			const FClientSequenceKey CacheKey{ Endpoint, Status.Key };
			PreloadStates.FindOrAdd(CacheKey) = Status.Value;

			UE_LOGFMT(LogSequencerPlaylists, Verbose, "Remote endpoint {Endpoint} reported {Result} for preload of '{Asset}'",
				Endpoint.ToString(), static_cast<int32>(Status.Value), Status.Key.ToString() );
		}
	}
}


#undef LOCTEXT_NAMESPACE
