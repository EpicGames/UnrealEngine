// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "SequencerPlaylistsSubsystem.generated.h"


enum class EConcertClientStatus : uint8;
enum class EConcertSequencerPreloadStatus : uint8;
struct FConcertSessionClientInfo;
struct FConcertSessionContext;
struct FConcertSequencerPreloadClientStatusMap;
class IConcertClientSession;
class SSequencerPlaylistPanel;
class ULevelSequence;
class USequencerPlaylist;
class USequencerPlaylistPlayer;


USTRUCT()
struct FSequencerPlaylistEditorHandle
{
	GENERATED_BODY()

	FSequencerPlaylistEditorHandle() {}

	explicit FSequencerPlaylistEditorHandle(SSequencerPlaylistPanel* InEditor)
		: Editor(InEditor)
	{
	}

	bool operator==(const FSequencerPlaylistEditorHandle& Other) const
	{
		return Editor == Other.Editor;
	}

	friend uint32 GetTypeHash(const FSequencerPlaylistEditorHandle& Handle)
	{
		return GetTypeHash(Handle.Editor);
	}

	SSequencerPlaylistPanel* Editor = nullptr;
};


UCLASS()
class USequencerPlaylistsSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin USubsystem interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem interface

	/** For every tab, we manage the lifetime of the associated Player and its transient Playlist. */
	USequencerPlaylistPlayer* CreatePlayerForEditor(TSharedRef<SSequencerPlaylistPanel> Editor);
	void NotifyEditorClosed(SSequencerPlaylistPanel* Editor);

	void UpdatePreloadSet();

	bool IsPreloadingActive() const;

	/** If any client has failed or is pending, the enum will indicate failure or pending respectively (otherwise, success).
	 * The accompanying text contains a (potentially multi-line) summary of which multi-user clients are in which states.
	 */
	TPair<EConcertSequencerPreloadStatus, FText> GetPreloadStatusForSequence(const FTopLevelAssetPath& SequencePath);

	TWeakPtr<IConcertClientSession> GetWeakSession() const
	{
		return WeakSession;
	}
protected:
	USequencerPlaylist* CreateTransientPlaylistForEditor(TSharedRef<SSequencerPlaylistPanel> Editor);

	void OnConcertSessionStartup(TSharedRef<IConcertClientSession> InSession);
	void OnConcertSessionShutdown(TSharedRef<IConcertClientSession> InSession);

	void HandleSessionClientChanged(IConcertClientSession& InSession, EConcertClientStatus InClientStatus, const FConcertSessionClientInfo& InClientInfo);
	void HandleSequencerPreloadStatusEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerPreloadClientStatusMap& InEvent);

protected:
	UPROPERTY(Transient)
	TMap<FSequencerPlaylistEditorHandle, TObjectPtr<UPackage>> EditorPackages;

	UPROPERTY(Transient)
	TMap<FSequencerPlaylistEditorHandle, TObjectPtr<USequencerPlaylistPlayer>> EditorPlayers;

	UPROPERTY(Transient)
	TMap<FSequencerPlaylistEditorHandle, TObjectPtr<USequencerPlaylist>> EditorPlaylists;

	UPROPERTY(Transient)
	TSet<TObjectPtr<ULevelSequence>> PreloadSequences;

	using FClientSequenceKey = TPair<FGuid, FTopLevelAssetPath>;
	TMap<FClientSequenceKey, EConcertSequencerPreloadStatus> PreloadStates;

	/** Weak pointer to the client session with which to send events. May be null or stale. */
	TWeakPtr<IConcertClientSession> WeakSession;
};
