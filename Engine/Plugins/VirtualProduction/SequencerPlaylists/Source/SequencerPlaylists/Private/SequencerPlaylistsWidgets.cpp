// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerPlaylistsWidgets.h"
#include "DetailsViewArgs.h"
#include "Misc/TransactionObjectEvent.h" // IWYU pragma: keep
#include "SequencerPlaylist.h"
#include "Editor.h"
#include "SequencerPlaylistItem_Sequence.h"
#include "Framework/Commands/UICommandList.h"
#include "SequencerPlaylistPlayer.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SequencerPlaylistsModule.h"
#include "IAssetTools.h"
#include "SequencerPlaylistsStyle.h"
#include "IDetailsView.h"
#include "SequencerPlaylistsSubsystem.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ConcertSequencerMessages.h"
#include "ContentBrowserModule.h"
#include "EditorFontGlyphs.h"
#include "FileHelpers.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"
#include "IContentBrowserSingleton.h"
#include "ISinglePropertyView.h"
#include "Layout/WidgetPath.h"
#include "LevelSequence.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/TextFilter.h"
#include "Misc/Paths.h"
#include "MovieScene.h"
#include "ScopedTransaction.h"
#include "SPositiveActionButton.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SequencerPlaylists"


const float SSequencerPlaylistPanel::DefaultWidth = 300.0f;
const FName SSequencerPlaylistPanel::ColumnName_HoverTransport(TEXT("HoverTransport"));
const FName SSequencerPlaylistPanel::ColumnName_Items(TEXT("Items"));
const FName SSequencerPlaylistPanel::ColumnName_Offset(TEXT("Offset"));
const FName SSequencerPlaylistPanel::ColumnName_Pause(TEXT("Pause"));
const FName SSequencerPlaylistPanel::ColumnName_Loop(TEXT("Loop"));
const FName SSequencerPlaylistPanel::ColumnName_Preload(TEXT("Preload"));
const FName SSequencerPlaylistPanel::ColumnName_HoverDetails(TEXT("HoverDetails"));


void SSequencerPlaylistPanel::Construct(const FArguments& InArgs, TSharedPtr<SDockTab> InContainingTab)
{
	WeakContainingTab = InContainingTab;

	SearchTextFilter = MakeShared<TTextFilter<const FSequencerPlaylistRowData&>>(
		TTextFilter<const FSequencerPlaylistRowData&>::FItemToStringArray::CreateSP(this, &SSequencerPlaylistPanel::GetSearchStrings));
	SearchTextFilter->OnChanged().AddSP(this, &SSequencerPlaylistPanel::RegenerateRows);

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Panel.Background"))
		]
		+ SOverlay::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					Construct_LeftToolbar()
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					Construct_RightToolbar()
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 8.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FSequencerPlaylistsStyle::Get().GetFontStyle("SequencerPlaylists.TitleFont"))
					.Text(TAttribute<FText>::CreateLambda([this]() { return FText::AsCultureInvariant(GetDisplayTitle()); }))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					Construct_Transport()
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(150.0f)
			.Padding(8.0f, 0.0f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SBox)
					.MinDesiredHeight(35.0f)
					[
						SNew(SMultiLineEditableTextBox)
						.Padding(0)
						.Margin(0)
						.AutoWrapText(true)
						.Style(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.EditableTextBox")
						.Font(FSequencerPlaylistsStyle::Get().GetFontStyle("SequencerPlaylists.DescriptionFont"))
						.HintText(LOCTEXT("PlaylistDescriptionHint", "<playlist description>"))
						.Text(TAttribute<FText>::CreateLambda([this]() {
							if (USequencerPlaylist* Playlist = GetPlaylist())
							{
								return Playlist->Description;
							}

							return FText::GetEmpty();
						}))
						.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType) {
							if (USequencerPlaylist* Playlist = GetPlaylist())
							{
								Playlist->Description = NewText;
							}
						})
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 6.0f)
			[
				Construct_AddSearchRow()
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				Construct_ItemListView()
			]
		]
	];

	if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
	{
		IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
		ConcertClient->OnSessionStartup().AddRaw(this, &SSequencerPlaylistPanel::OnConcertSessionStartup);
		ConcertClient->OnSessionShutdown().AddRaw(this, &SSequencerPlaylistPanel::OnConcertSessionShutdown);
	}
}


SSequencerPlaylistPanel::~SSequencerPlaylistPanel()
{
	GEditor->GetEditorSubsystem<USequencerPlaylistsSubsystem>()->NotifyEditorClosed(this);

	if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
	{
		IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
		ConcertClient->OnSessionStartup().RemoveAll(this);
		ConcertClient->OnSessionShutdown().RemoveAll(this);
	}
}


FString SSequencerPlaylistPanel::GetDisplayTitle()
{
	FString DisplayTitle;

	USequencerPlaylist* WorkingPlaylist = GetPlaylist();

	if (USequencerPlaylist* LoadedFrom = WeakLoadedPlaylist.Get())
	{
		DisplayTitle = LoadedFrom->GetName();
		if (WorkingPlaylist && WorkingPlaylist->GetPackage() && WorkingPlaylist->GetPackage()->IsDirty())
		{
			DisplayTitle.AppendChar(TEXT('*'));
		}
	}
	else if (WorkingPlaylist)
	{
		DisplayTitle = WorkingPlaylist->GetName();
	}

	return DisplayTitle;
}


void SSequencerPlaylistPanel::SetPlayer(USequencerPlaylistPlayer* Player)
{
	WeakPlayer = Player;

	RegenerateRows();
}


bool SSequencerPlaylistPanel::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjectContexts)
	{
		UObject* Object = TransactionObjectPair.Key;
		while (Object != nullptr)
		{
			if (Object->GetClass()->IsChildOf(USequencerPlaylist::StaticClass()))
			{
				return true;
			}
			Object = Object->GetOuter();
		}
	}

	return false;
}


void SSequencerPlaylistPanel::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		RegenerateRows();
	}
}


void SSequencerPlaylistPanel::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}


TSharedRef<SWidget> SSequencerPlaylistPanel::Construct_LeftToolbar()
{
	FSequencerPlaylistsModule& Module = static_cast<FSequencerPlaylistsModule&>(ISequencerPlaylistsModule::Get());
	FSlimHorizontalToolBarBuilder ToolBarBuilder(Module.GetCommandList(), FMultiBoxCustomization::None, nullptr, true);
	ToolBarBuilder.SetStyle(&FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.MainToolbar");

	ToolBarBuilder.BeginSection("Playlists");
	{
		ToolBarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SSequencerPlaylistPanel::OnNewPlaylist)),
			NAME_None,
			LOCTEXT("NewPlaylist", "New Playlist"),
			LOCTEXT("NewPlaylistTooltip", "New Playlist"),
			FSlateIcon(FSequencerPlaylistsStyle::Get().GetStyleSetName(), "SequencerPlaylists.NewPlaylist"));

		ToolBarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SSequencerPlaylistPanel::OnSavePlaylist)),
			NAME_None,
			LOCTEXT("SavePlaylist", "Save Playlist"),
			LOCTEXT("SavePlaylistTooltip", "Save Playlist"),
			FSlateIcon(FSequencerPlaylistsStyle::Get().GetStyleSetName(), "SequencerPlaylists.SavePlaylist"));

		ToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateLambda([this]() {
				FMenuBuilder MenuBuilder(true, nullptr);
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SavePlaylistAs", "Save Playlist As..."),
					LOCTEXT("SavePlaylistAsTooltip", "Save Playlist as..."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SSequencerPlaylistPanel::OnSavePlaylistAs))
				);
				return MenuBuilder.MakeWidget();
			}),
			LOCTEXT("SavePlaylistOptions", "Save Playlist Options"),
			LOCTEXT("SavePlaylistOptionsTooltip", "Save Playlist options"),
			TAttribute<FSlateIcon>(),
			true);

		ToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SSequencerPlaylistPanel::BuildOpenPlaylistMenu),
			LOCTEXT("OpenPlaylist", "Load Playlist"),
			LOCTEXT("OpenPlaylistTooltip", "Load Playlist"),
			FSlateIcon(FSequencerPlaylistsStyle::Get().GetStyleSetName(), "SequencerPlaylists.OpenPlaylist"));
	}
	ToolBarBuilder.EndSection();

	return ToolBarBuilder.MakeWidget();
}


TSharedRef<SWidget> SSequencerPlaylistPanel::Construct_RightToolbar()
{
	FSequencerPlaylistsModule& Module = static_cast<FSequencerPlaylistsModule&>(ISequencerPlaylistsModule::Get());
	FSlimHorizontalToolBarBuilder ToolBarBuilder(Module.GetCommandList(), FMultiBoxCustomization::None, nullptr, true);
	ToolBarBuilder.SetStyle(&FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.MainToolbar");

	ToolBarBuilder.BeginSection("PlayMode");
	{
		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this]() {
					bPlayMode = !bPlayMode;
				}),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([this]() {
					return InPlayMode() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			),
			NAME_None,
			LOCTEXT("TogglePlayMode", "Toggle Play Mode"),
			LOCTEXT("TogglePlayModeTooltip", "Enable or disable Play Mode. Play Mode disables editing and provides larger play buttons for easier playback."),
			FSlateIcon(FSequencerPlaylistsStyle::Get().GetStyleSetName(), "SequencerPlaylists.PlayMode"),
			EUserInterfaceActionType::ToggleButton);
	}
	ToolBarBuilder.EndSection();

	return ToolBarBuilder.MakeWidget();
}


TSharedRef<SWidget> SSequencerPlaylistPanel::Construct_Transport()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.PlayReverse")
			.ContentPadding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SSequencerPlaylistPanel::HandleClicked_PlayAllReverse)
			.ToolTipText(LOCTEXT("PlayAllReverseButtonTooltip", "Play all items in reverse simultaneously."))
			[
				SNew(SImage)
				.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.PlayReverse"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.Pause")
			.ContentPadding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SSequencerPlaylistPanel::HandleClicked_PauseAll)
			.ToolTipText(LOCTEXT("PauseAllButtonTooltip", "Pause all items."))
			[
				SNew(SImage)
				.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Pause"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.Play")
			.ContentPadding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SSequencerPlaylistPanel::HandleClicked_PlayAll)
			.ToolTipText(LOCTEXT("PlayAllButtonTooltip", "Play all items simultaneously."))
			[
				SNew(SImage)
				.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Play"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.Stop")
			.ContentPadding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SSequencerPlaylistPanel::HandleClicked_StopAll)
			.ToolTipText(LOCTEXT("StopAllButtonTooltip", "Stop all currently playing items."))
			[
				SNew(SImage)
				.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Stop"))
				.DesiredSizeOverride(FVector2D(18.0f, 18.0f))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.Reset")
			.ContentPadding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SSequencerPlaylistPanel::HandleClicked_ResetAll)
			.ToolTipText(LOCTEXT("ResetAllButtonTooltip", "Stop all currently playing items and re-hold, if specified."))
			[
				SNew(SImage)
				.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Reset"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
}


TSharedRef<SWidget> SSequencerPlaylistPanel::Construct_AddSearchRow()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SPositiveActionButton)
			.OnClicked(this, &SSequencerPlaylistPanel::HandleClicked_AddSequence)
			.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			.Text(LOCTEXT("AddItemButton", "Item"))
			.ToolTipText(LOCTEXT("AddItemButtonTooltip", "Add a new level sequence Playlist item"))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(8.0f, 0.0f)
		[
			SAssignNew(SearchBox, SSearchBox)
			.HintText(LOCTEXT("SearchHint", "Search Playlist Items"))
			.OnTextChanged(this, &SSequencerPlaylistPanel::OnSearchTextChanged)
			.DelayChangeNotificationsWhileTyping(true)
		];
}


TSharedRef<SWidget> SSequencerPlaylistPanel::Construct_ItemListView()
{
	TSharedRef<SWidget> Widget = SAssignNew(ItemListView, SListView<TSharedPtr<FSequencerPlaylistRowData>>)
		.SelectionMode(ESelectionMode::Single) // See TODO in HandleAcceptDrop
		.ListItemsSource(&ItemRows)
		.OnGenerateRow_Lambda([this](TSharedPtr<FSequencerPlaylistRowData> InData, const TSharedRef<STableViewBase>& OwnerTableView)
			{
				return SNew(SSequencerPlaylistItemWidget, InData, OwnerTableView)
					.PlayMode(this, &SSequencerPlaylistPanel::InPlayMode)
					.IsPlaying_Lambda([this, WeakItem = InData->WeakItem]() -> bool {
						USequencerPlaylistItem* Item = WeakItem.Get();
						USequencerPlaylistPlayer* Player = WeakPlayer.Get();
						if (Item && Player)
						{
							return Player->GetPlaybackState(Item).bIsPlaying;
						}
						else
						{
							return false;
						}
					})
					.IsPaused_Lambda([this, WeakItem = InData->WeakItem]() -> bool {
						USequencerPlaylistItem* Item = WeakItem.Get();
						USequencerPlaylistPlayer* Player = WeakPlayer.Get();
						if (Item && Player)
						{
							return Player->GetPlaybackState(Item).bIsPaused;
						}
						else
						{
							return false;
						}
					})
					.OnPlayClicked(this, &SSequencerPlaylistPanel::HandleClicked_Item_Play)
					.OnPlayReverseClicked(this, &SSequencerPlaylistPanel::HandleClicked_Item_PlayReverse)
					.OnPauseClicked(this, &SSequencerPlaylistPanel::HandleClicked_Item_Pause)
					.OnStopClicked(this, &SSequencerPlaylistPanel::HandleClicked_Item_Stop)
					.OnResetClicked(this, &SSequencerPlaylistPanel::HandleClicked_Item_Reset)
					.OnRemoveClicked(this, &SSequencerPlaylistPanel::HandleClicked_Item_Remove)
					.OnIsPropertyVisible(this, &SSequencerPlaylistPanel::HandleItemDetailsIsPropertyVisible)
					.OnCanAcceptDrop(this, &SSequencerPlaylistPanel::HandleCanAcceptDrop)
					.OnAcceptDrop(this, &SSequencerPlaylistPanel::HandleAcceptDrop);
			})
		.HeaderRow(
			SAssignNew(HeaderRow, SHeaderRow)
			+ SHeaderRow::Column(ColumnName_HoverTransport)
				.DefaultLabel(FText::GetEmpty())
				.FillSized(60.0f)
			+ SHeaderRow::Column(ColumnName_Preload)
				.DefaultLabel(LOCTEXT("ColumnLabelPreload", "Preload"))
				.FillSized(60.0f)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)
			+ SHeaderRow::Column(ColumnName_Items)
				.DefaultLabel(LOCTEXT("ColumnLabelItems", "Playlist Items"))
				.FillWidth(1.0f)
			+ SHeaderRow::Column(ColumnName_Offset)
				.DefaultLabel(LOCTEXT("ColumnLabelOffset", "Offset"))
				.FillSized(45.0f)
			+ SHeaderRow::Column(ColumnName_Pause)
				.DefaultLabel(LOCTEXT("ColumnLabelPause", "Pause"))
				.FillSized(45.0f)
				.HAlignCell(HAlign_Center)
			+ SHeaderRow::Column(ColumnName_Loop)
				.DefaultLabel(LOCTEXT("ColumnLabelLoop", "Loop"))
				.FillSized(80.0f)
			+ SHeaderRow::Column(ColumnName_HoverDetails)
				.DefaultLabel(FText::GetEmpty())
				.FillSized(30.0f)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)
		);

	if (ensure(HeaderRow))
	{
		bool bPreloadVisible = false;
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			bPreloadVisible = ConcertClient->GetCurrentSession().IsValid();
		}
		HeaderRow->SetShowGeneratedColumn(ColumnName_Preload, bPreloadVisible);
	}
	return Widget;
}


USequencerPlaylist* SSequencerPlaylistPanel::GetCheckedPlaylist()
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return nullptr;
	}

	USequencerPlaylist* Playlist = Player->GetPlaylist();
	ensure(Playlist);
	return Playlist;
}


USequencerPlaylist* SSequencerPlaylistPanel::GetPlaylist()
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!Player)
	{
		return nullptr;
	}

	return Player->GetPlaylist();
}


void SSequencerPlaylistPanel::RegenerateRows()
{
	USequencerPlaylist* Playlist = GetCheckedPlaylist();
	if (!Playlist)
	{
		return;
	}

	const int32 ItemCount = Playlist->Items.Num();
	ItemRows.Empty(ItemCount);
	for (int32 ItemIndex = 0; ItemIndex < ItemCount; ++ItemIndex)
	{
		USequencerPlaylistItem* Item = Playlist->Items[ItemIndex];
		FSequencerPlaylistRowData Row(ItemIndex, Item);
		if (SearchTextFilter->PassesFilter(Row))
		{
			ItemRows.Emplace(MakeShared<FSequencerPlaylistRowData>(Row));
		}
	}

	ItemListView->RequestListRefresh();
}


TSharedRef<SWidget> SSequencerPlaylistPanel::BuildOpenPlaylistMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("OpenPlaylistNoAssetsWarning", "No Playlists Found");
		AssetPickerConfig.Filter.ClassPaths.Add(USequencerPlaylist::StaticClass()->GetClassPathName());
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SSequencerPlaylistPanel::OnLoadPlaylist);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("OpenPlaylistMenuSection", "Load Playlist"));
	{
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.MinDesiredWidth(400.f)
			.MinDesiredHeight(400.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


static bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName)
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	{
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InNewNameSuggestion;
		SaveAssetDialogConfig.AssetClassNames.Add(USequencerPlaylist::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SavePlaylistDialogTitle", "Save Sequencer Playlist");
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (!SaveObjectPath.IsEmpty())
	{
		OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		return true;
	}

	return false;
}


bool GetSavePlaylistPackageName(FString& OutName)
{
	// TODO
	//USequencerPlaylistsSettings* ConfigSettings = GetMutableDefault<USequencerPlaylistsSettings>();

	FDateTime Today = FDateTime::Now();

	TMap<FString, FStringFormatArg> FormatArgs;
	FormatArgs.Add(TEXT("date"), Today.ToString());

	// determine default package path
	const FString DefaultSaveDirectory;// = FString::Format(*ConfigSettings->GetPresetSaveDir().Path, FormatArgs);

	FString DialogStartPath;
	FPackageName::TryConvertFilenameToLongPackageName(DefaultSaveDirectory, DialogStartPath);
	if (DialogStartPath.IsEmpty())
	{
		DialogStartPath = TEXT("/Game");
	}

	// determine default asset name
	FString DefaultName = LOCTEXT("NewPlaylistDefaultName", "NewSequencerPlaylist").ToString();

	FString UniquePackageName;
	FString UniqueAssetName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(DialogStartPath / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);

	FString DialogStartName = FPaths::GetCleanFilename(UniqueAssetName);

	FString UserPackageName;
	FString NewPackageName;

	// get destination for asset
	bool bFilenameValid = false;
	while (!bFilenameValid)
	{
		if (!OpenSaveDialog(DialogStartPath, DialogStartName, UserPackageName))
		{
			return false;
		}

		NewPackageName = FString::Format(*UserPackageName, FormatArgs);

		FText OutError;
		bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError);
	}

	//ConfigSettings->PresetSaveDir.Path = FPackageName::GetLongPackagePath(UserPackageName);
	//ConfigSettings->SaveConfig();
	OutName = MoveTemp(NewPackageName);
	return true;
}


void SSequencerPlaylistPanel::OnSavePlaylist()
{
	if (!WeakLoadedPlaylist.IsValid() ||
		!WeakLoadedPlaylist->GetPackage() ||
		!WeakLoadedPlaylist->GetPackage()->GetLoadedPath().HasPackageName())
	{
		OnSavePlaylistAs();
		return;
	}

	FString PackageName = WeakLoadedPlaylist->GetPackage()->GetLoadedPath().GetPackageName();
	SavePlaylistAs(PackageName);
}


void SSequencerPlaylistPanel::OnSavePlaylistAs()
{
	FString PackageName;
	if (!GetSavePlaylistPackageName(PackageName))
	{
		return;
	}

	SavePlaylistAs(PackageName);
}

void SSequencerPlaylistPanel::SavePlaylistAs(const FString& PackageName)
{
	USequencerPlaylist* Playlist = GetCheckedPlaylist();
	if (!Playlist)
	{
		return;
	}

	const FString AssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage* Package = CreatePackage(*PackageName);
	USequencerPlaylist* NewPlaylist = NewObject<USequencerPlaylist>(Package, *AssetName, RF_Public | RF_Standalone | RF_Transactional);
	if (NewPlaylist)
	{
		NewPlaylist->Description = Playlist->Description;

		for (USequencerPlaylistItem* Item : Playlist->Items)
		{
			NewPlaylist->Items.Add(DuplicateObject<USequencerPlaylistItem>(Item, NewPlaylist));
		}

		NewPlaylist->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(NewPlaylist);

		FEditorFileUtils::PromptForCheckoutAndSave({ Package }, false, false);
	}

	WeakLoadedPlaylist = NewPlaylist;
	if (Playlist->GetPackage())
	{
		Playlist->GetPackage()->ClearDirtyFlag();
	}
}


void SSequencerPlaylistPanel::OnLoadPlaylist(const FAssetData& InPreset)
{
	FSlateApplication::Get().DismissAllMenus();
	LoadPlaylist(CastChecked<USequencerPlaylist>(InPreset.GetAsset()));
}


void SSequencerPlaylistPanel::LoadPlaylist(USequencerPlaylist* PlaylistToLoad)
{
	if (!ensure(PlaylistToLoad))
	{
		return;
	}

	USequencerPlaylist* Playlist = GetCheckedPlaylist();
	if (!Playlist)
	{
		return;
	}

	Playlist->Description = PlaylistToLoad->Description;

	Playlist->Items.Empty();
	for (USequencerPlaylistItem* Item : PlaylistToLoad->Items)
	{
		Playlist->Items.Add(DuplicateObject<USequencerPlaylistItem>(Item, Playlist));
	}

	RegenerateRows();

	WeakLoadedPlaylist = PlaylistToLoad;
	if (Playlist->GetPackage())
	{
		Playlist->GetPackage()->ClearDirtyFlag();
	}

	if (USequencerPlaylistsSubsystem* Subsystem = GEditor->GetEditorSubsystem<USequencerPlaylistsSubsystem>())
	{
		Subsystem->UpdatePreloadSet();
	}
}


void SSequencerPlaylistPanel::OnNewPlaylist()
{
	USequencerPlaylist* Playlist = GetCheckedPlaylist();
	if (!Playlist)
	{
		return;
	}

	Playlist->Description = FText::GetEmpty();
	Playlist->Items.Empty();
	RegenerateRows();

	WeakLoadedPlaylist = nullptr;
}


void SSequencerPlaylistPanel::GetSearchStrings(const FSequencerPlaylistRowData& Row, TArray<FString>& OutSearchStrings)
{
	if (USequencerPlaylistItem* Item = Row.WeakItem.Get())
	{
		OutSearchStrings.Add(Item->GetDisplayName().ToString());
	}
}


void SSequencerPlaylistPanel::OnSearchTextChanged(const FText& InFilterText)
{
	SearchTextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError(SearchTextFilter->GetFilterErrorText());
}


FReply SSequencerPlaylistPanel::HandleClicked_PlayAll()
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->PlayAll();
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_PlayAllReverse()
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->PlayAll(ESequencerPlaylistPlaybackDirection::Reverse);
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_PauseAll()
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->PauseAll();
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_StopAll()
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->StopAll();
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_ResetAll()
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->ResetAll();
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_AddSequence()
{
	USequencerPlaylist* Playlist = GetCheckedPlaylist();
	if (!Playlist)
	{
		return FReply::Unhandled();
	}

	FScopedTransaction Transaction(LOCTEXT("AddSequenceTransaction", "Add Sequence Item To Playlist"));
	Playlist->Modify();

	USequencerPlaylistItem_Sequence* NewItem = NewObject<USequencerPlaylistItem_Sequence>(Playlist);
	NewItem->SetFlags(RF_Transactional);
	Playlist->Items.Add(NewItem);
	RegenerateRows();

	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_Item_Play(SSequencerPlaylistItemWidget& ItemWidget)
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->PlayItem(ItemWidget.GetItem());
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_Item_PlayReverse(SSequencerPlaylistItemWidget& ItemWidget)
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->PlayItem(ItemWidget.GetItem(), ESequencerPlaylistPlaybackDirection::Reverse);
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_Item_Pause(SSequencerPlaylistItemWidget& ItemWidget)
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->PauseItem(ItemWidget.GetItem());
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_Item_Stop(SSequencerPlaylistItemWidget& ItemWidget)
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->StopItem(ItemWidget.GetItem());
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_Item_Reset(SSequencerPlaylistItemWidget& ItemWidget)
{
	USequencerPlaylistPlayer* Player = WeakPlayer.Get();
	if (!ensure(Player))
	{
		return FReply::Unhandled();
	}

	Player->ResetItem(ItemWidget.GetItem());
	return FReply::Handled();
}


FReply SSequencerPlaylistPanel::HandleClicked_Item_Remove(SSequencerPlaylistItemWidget& ItemWidget)
{
	USequencerPlaylist* Playlist = GetCheckedPlaylist();
	if (!Playlist)
	{
		return FReply::Unhandled();
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveSequenceTransaction", "Remove Sequence Item From Playlist"));
	Playlist->Modify();

	ensure(Playlist->Items.RemoveSingle(ItemWidget.GetItem()));
	RegenerateRows();
	return FReply::Handled();
}


bool SSequencerPlaylistPanel::HandleItemDetailsIsPropertyVisible(const FPropertyAndParent& PropertyAndParent)
{
	static TMap<FName, FName> PropertyNameToColumnName;
	static bool bMapInitialized = false;
	if (bMapInitialized == false)
	{
		PropertyNameToColumnName.Add("Sequence", ColumnName_Items);

		PropertyNameToColumnName.Add("StartFrameOffset", ColumnName_Offset);
		PropertyNameToColumnName.Add("EndFrameOffset", ColumnName_Offset);
		PropertyNameToColumnName.Add("NumLoops", ColumnName_Loop);

		bMapInitialized = true;
	}

	if (const FName* ColumnName = PropertyNameToColumnName.Find(PropertyAndParent.Property.GetFName()))
	{
		if (ItemListView->GetHeaderRow()->IsColumnVisible(*ColumnName))
		{
			return false;
		}
	}

	return true;
}


TOptional<EItemDropZone> SSequencerPlaylistPanel::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FSequencerPlaylistRowData> RowData)
{
	TSharedPtr<FSequencerPlaylistItemDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FSequencerPlaylistItemDragDropOp>();
	if (DragDropOperation.IsValid())
	{
		return DropZone == EItemDropZone::OntoItem ? EItemDropZone::BelowItem : DropZone;
	}

	return TOptional<EItemDropZone>();
}


FReply SSequencerPlaylistPanel::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FSequencerPlaylistRowData> RowData)
{
	USequencerPlaylist* Playlist = GetCheckedPlaylist();
	if (!Playlist)
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FSequencerPlaylistItemDragDropOp> Operation = DragDropEvent.GetOperationAs<FSequencerPlaylistItemDragDropOp>();
	if (Operation.IsValid())
	{
		// TODO: Not currently handling (potentially disjoint) multi-select,
		// in part because there's no Algo::StablePartition.
		// The ListView is set to ESelectionMode::Single for the time being.
		if (ensure(Operation->SelectedItems.Num() == 1))
		{
			const int32 SrcIndex = Operation->SelectedItems[0]->PlaylistIndex;
			const int32 DropTargetIndex = RowData->PlaylistIndex;

			if (SrcIndex == DropTargetIndex)
			{
				return FReply::Handled();
			}

			int32 DestIndexAdjustment = 0;
			if (DropZone == EItemDropZone::BelowItem)
			{
				DestIndexAdjustment += 1;
			}

			if (SrcIndex < DropTargetIndex)
			{
				DestIndexAdjustment -= 1;
			}

			const int32 DestIndex = DropTargetIndex + DestIndexAdjustment;
			TArray<TObjectPtr<USequencerPlaylistItem>>& Items = Playlist->Items;
			if (ensure(Items.IsValidIndex(SrcIndex)) && ensure(Items.IsValidIndex(DestIndex)))
			{
				USequencerPlaylistItem* ItemToMove = Items[SrcIndex];
				Items.RemoveAt(SrcIndex);
				Items.Insert(ItemToMove, DestIndex);
				RegenerateRows();
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}


void SSequencerPlaylistPanel::OnConcertSessionStartup(TSharedRef<IConcertClientSession> InSession)
{
	if (ensure(HeaderRow))
	{
		HeaderRow->SetShowGeneratedColumn(ColumnName_Preload, true);
	}
}

void SSequencerPlaylistPanel::OnConcertSessionShutdown(TSharedRef<IConcertClientSession> InSession)
{
	if (ensure(HeaderRow))
	{
		HeaderRow->SetShowGeneratedColumn(ColumnName_Preload, false);
	}
}


TSharedRef<FSequencerPlaylistItemDragDropOp> FSequencerPlaylistItemDragDropOp::New(const TArray<TSharedPtr<FSequencerPlaylistRowData>>& InSelectedItems)
{
	TSharedRef<FSequencerPlaylistItemDragDropOp> Operation = MakeShared<FSequencerPlaylistItemDragDropOp>();

	Operation->SelectedItems = InSelectedItems;

	Operation->MouseCursor = EMouseCursor::GrabHandClosed;
	Operation->Decorator = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("ItemDragHint", "Move {0} {0}|plural(one=item,other=items)"), InSelectedItems.Num()))
			]
		];

	Operation->Construct();

	return Operation;
}


FSequencerPlaylistItemDragDropOp::~FSequencerPlaylistItemDragDropOp()
{
}


const FText SSequencerPlaylistItemWidget::PlayItemTooltipText(LOCTEXT("PlayItemTooltip", "Play just this item."));
const FText SSequencerPlaylistItemWidget::PlayReverseItemTooltipText(LOCTEXT("PlayReverseItemTooltip", "Play just this item in reverse."));
const FText SSequencerPlaylistItemWidget::PauseItemTooltipText(LOCTEXT("PauseItemTooltip", "Pause just this item. If not recording or playing back, creates a hold section for this item."));
const FText SSequencerPlaylistItemWidget::StopItemTooltipText(LOCTEXT("StopItemTooltip", "Stop all sequences created by this item."));
const FText SSequencerPlaylistItemWidget::ResetItemTooltipText(LOCTEXT("ResetItemTooltip", "Stop all sequences created by this item and re-hold, if specified."));


void SSequencerPlaylistItemWidget::Construct(const FArguments& InArgs, TSharedPtr<FSequencerPlaylistRowData> InRowData, const TSharedRef<STableViewBase>& OwnerTableView)
{
	check(InRowData && InRowData->WeakItem.IsValid());
	RowData = InRowData;

	LoopMode = InRowData->WeakItem->NumLoops == 0 ? ELoopMode::None : ELoopMode::Finite;

	PlayMode = InArgs._PlayMode;
	IsPlaying = InArgs._IsPlaying;
	IsPaused = InArgs._IsPaused;

	PlayClickedDelegate = InArgs._OnPlayClicked;
	PlayReverseClickedDelegate = InArgs._OnPlayReverseClicked;
	PauseClickedDelegate = InArgs._OnPauseClicked;
	StopClickedDelegate = InArgs._OnStopClicked;
	ResetClickedDelegate = InArgs._OnResetClicked;
	RemoveClickedDelegate = InArgs._OnRemoveClicked;

	IsPropertyVisibleDelegate = InArgs._OnIsPropertyVisible;

	FSuperRowType::Construct(
		FSuperRowType::FArguments()
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
			.OnDragDetected(this, &SSequencerPlaylistItemWidget::HandleDragDetected)
			.Style(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.ItemRow")
		, OwnerTableView);
}


void SSequencerPlaylistItemWidget::ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent)
{
	// We wrap InContent with an overlay which facilitates dimming the row, obstructing hit test of
	// the controls behind, and centering Play Mode transport controls over the row on hover.
	Content = InContent;

	ChildSlot
	.Padding(InPadding)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.Padding(1.0f)
		[
			SNew(SImage)
			.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.ItemRow.BackgroundOuter"))
		]
		+ SOverlay::Slot()
		.Padding(8.0f, 4.0f)
		[
			SNew(SImage)
			.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.ItemRow.BackgroundInner"))
		]
		+ SOverlay::Slot()
		.Padding(0.0f, 4.0f)
		[
			InContent
		]
		+ SOverlay::Slot()
		.Padding(1.0f)
		[
			SNew(SImage)
			.Visibility(this, &SSequencerPlaylistItemWidget::GetRowDimmingVisibility)
			.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Item.Dim"))
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SSequencerPlaylistItemWidget::GetPlayModeTransportVisibility)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.PlayReverse")
				.ContentPadding(6.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([this]() { return PlayReverseClickedDelegate.Execute(*this); })
				.ToolTipText(PlayReverseItemTooltipText)
				[
					SNew(SImage)
					.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.PlayReverse"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.Pause")
				.ContentPadding(6.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([this]() { return PauseClickedDelegate.Execute(*this); })
				.ToolTipText(PauseItemTooltipText)
				[
					SNew(SImage)
					.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Pause"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.Play")
				.ContentPadding(6.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([this]() { return PlayClickedDelegate.Execute(*this); })
				.ToolTipText(PlayItemTooltipText)
				[
					SNew(SImage)
					.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Play"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.Stop")
				.ContentPadding(6.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([this]() { return StopClickedDelegate.Execute(*this); })
				.ToolTipText(StopItemTooltipText)
				[
					SNew(SImage)
					.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Stop"))
					.DesiredSizeOverride(FVector2D(18.0f, 18.0f))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.TransportButton.Reset")
				.ContentPadding(6.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([this]() { return ResetClickedDelegate.Execute(*this); })
				.ToolTipText(ResetItemTooltipText)
				[
					SNew(SImage)
					.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Reset"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	];

	InnerContentSlot = &ChildSlot.AsSlot();
}


TSharedRef<SWidget> SSequencerPlaylistItemWidget::GenerateWidgetForColumn(const FName& ColumnName)
{
	USequencerPlaylistItem* Item = GetItem();

	if (!ensure(Item))
	{
		return SNullWidget::NullWidget;
	}

	TWeakObjectPtr<USequencerPlaylistItem> WeakItem = Item;

	static const FName PropertyEditorModuleName("PropertyEditor");
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
	FSinglePropertyParams SinglePropParams;
	SinglePropParams.NamePlacement = EPropertyNamePlacement::Hidden;

	if (ColumnName == SSequencerPlaylistPanel::ColumnName_HoverTransport)
	{
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(12.0f, 4.0f, 0.0f, 2.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.HoverTransport.PlayReverse")
					.ContentPadding(0)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility_Lambda([this]() { return (IsHovered() && !InPlayMode()) ? EVisibility::Visible : EVisibility::Hidden; })
					.OnClicked_Lambda([this]() { return PlayReverseClickedDelegate.Execute(*this); })
					.ToolTipText(PlayReverseItemTooltipText)
					[
						SNew(SImage)
						.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.PlayReverse.Small"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.HoverTransport.Play")
					.ContentPadding(0)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility_Lambda([this]() { return (IsHovered() && !InPlayMode()) ? EVisibility::Visible : EVisibility::Hidden; })
					.OnClicked_Lambda([this]() { return PlayClickedDelegate.Execute(*this); })
					.ToolTipText(PlayItemTooltipText)
					[
						SNew(SImage)
						.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Play.Small"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
			+ SVerticalBox::Slot()
			.Padding(12.0f, 2.0f, 0.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.HoverTransport.Stop")
					.ContentPadding(0)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility_Lambda([this]() { return (IsPlaying.Get() || (IsHovered() && !InPlayMode())) ? EVisibility::Visible : EVisibility::Hidden; })
					.ForegroundColor_Lambda([this]() { return IsPlaying.Get() ? FStyleColors::AccentRed : FSlateColor::UseStyle(); })
					.OnClicked_Lambda([this]() { return StopClickedDelegate.Execute(*this); })
					.ToolTipText(StopItemTooltipText)
					[
						SNew(SImage)
						.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Stop.Small"))
						.DesiredSizeOverride(FVector2D(14.0f, 14.0f))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ButtonStyle(FSequencerPlaylistsStyle::Get(), "SequencerPlaylists.HoverTransport.Reset")
					.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 1.0f))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility_Lambda([this]() { return (IsHovered() && !InPlayMode()) ? EVisibility::Visible : EVisibility::Hidden; })
					.OnClicked_Lambda([this]() { return ResetClickedDelegate.Execute(*this); })
					.ToolTipText(ResetItemTooltipText)
					[
						SNew(SImage)
						.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Reset.Small"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
	}

	if (ColumnName == SSequencerPlaylistPanel::ColumnName_Items)
	{
		USequencerPlaylistItem_Sequence* SequenceItem =
			CastChecked<USequencerPlaylistItem_Sequence>(Item);

		TSharedPtr<ISinglePropertyView> PropView = PropertyEditorModule.CreateSingleProperty(SequenceItem,
			USequencerPlaylistItem_Sequence::GetSequencePropertyName(),
			SinglePropParams);

		return SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
			[
				PropView.ToSharedRef()
			];
	}

	if (ColumnName == SSequencerPlaylistPanel::ColumnName_Offset)
	{
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SNumericEntryBox<int32>)
				.AllowSpin(true)
				.ToolTipText(LOCTEXT("StartFrameOffsetTooltip", "Number of frames by which to clip the in point of sections played from this item. Will also affect the first frame for hold."))
				.MinSliderValue(0)
				.MaxSliderValue(this, &SSequencerPlaylistItemWidget::GetItemLengthDisplayFrames)
				.Value(TAttribute<TOptional<int32>>::CreateLambda([WeakItem]() { return WeakItem.IsValid() ? WeakItem->StartFrameOffset : TOptional<int32>(); }))
				.OnValueChanged_Lambda([WeakItem](int32 NewValue) {
					if (WeakItem.IsValid())
					{
						WeakItem->StartFrameOffset = NewValue;
					}
				})
				.OnValueCommitted_Lambda([WeakItem](int32 NewValue, ETextCommit::Type) {
					if (WeakItem.IsValid())
					{
						WeakItem->StartFrameOffset = NewValue;
					}
				})
			]
			+ SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SNew(SNumericEntryBox<int32>)
				.AllowSpin(true)
				.ToolTipText(LOCTEXT("EndFrameOffsetTooltip", "Number of frames by which to clip the out point of sections played from this item."))
				.MinSliderValue(0)
				.MaxSliderValue(this, &SSequencerPlaylistItemWidget::GetItemLengthDisplayFrames)
				.Value(TAttribute<TOptional<int32>>::CreateLambda([WeakItem]() { return WeakItem.IsValid() ? WeakItem->EndFrameOffset : TOptional<int32>(); }))
				.OnValueChanged_Lambda([WeakItem](int32 NewValue) {
					if (WeakItem.IsValid())
					{
						WeakItem->EndFrameOffset = NewValue;
					}
				})
				.OnValueCommitted_Lambda([WeakItem](int32 NewValue, ETextCommit::Type) {
					if (WeakItem.IsValid())
					{
						WeakItem->EndFrameOffset = NewValue;
					}
				})
			];
	}

	if (ColumnName == SSequencerPlaylistPanel::ColumnName_Pause)
	{
		TSharedRef<SWidget> PauseToggle = SNew(SCheckBox)
			.Padding(FMargin(4.0f, 2.0f))
			.HAlign(HAlign_Center)
			.ToolTipText(LOCTEXT("TogglePauseTooltip", "Enable or disable pause. Pause will infinitely hold the first frame of this item until manually played. Items are put into a pause state at the start of a take, or manually by hitting \"Reset.\""))
			.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
			.IsChecked_Lambda([this]() { return IsPaused.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InState) { PauseClickedDelegate.Execute(*this); })
			[
				SNew(SImage)
				.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Pause"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];

		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.0f, 1.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				PauseToggle
			]
			+ SVerticalBox::Slot()
			[
				SNullWidget::NullWidget
			];
	}

	if (ColumnName == SSequencerPlaylistPanel::ColumnName_Loop)
	{
		TSharedRef<SWidget> LoopModeCombo = SNew(SComboButton)
			.ContentPadding(0)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.ToolTipText(LOCTEXT("LoopModeComboTooltip", "Toggle between looping a specified number of times or not looping."))
			.OnGetMenuContent_Lambda([this]() {
				FMenuBuilder MenuBuilder(true, nullptr);
				MenuBuilder.AddMenuEntry(
					LOCTEXT("LoopModeNone", "No Loop"),
					LOCTEXT("LoopModeNoneTooltip", "No Loop"),
					FSlateIcon(FSequencerPlaylistsStyle::Get().GetStyleSetName(), "SequencerPlaylists.Loop.Disabled"),
					FUIAction(
						FExecuteAction::CreateLambda([this]() { SetLoopMode(ELoopMode::None); }),
						FCanExecuteAction(),
						FGetActionCheckState::CreateLambda([this]() { return LoopMode == ELoopMode::None ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					),
					NAME_None,
					EUserInterfaceActionType::Check
				);
				MenuBuilder.AddMenuEntry(
					// U+1D62F = "Mathematical Sans-Serif Italic Small N"
					LOCTEXT("LoopModeFinite", "Loop [\U0001D62F] Times"),
					LOCTEXT("LoopModeFiniteTooltip", "Loop [\U0001D62F] Times"),
					FSlateIcon(FSequencerPlaylistsStyle::Get().GetStyleSetName(), "SequencerPlaylists.Loop.Finite"),
					FUIAction(
						FExecuteAction::CreateLambda([this]() { SetLoopMode(ELoopMode::Finite); }),
						FCanExecuteAction(),
						FGetActionCheckState::CreateLambda([this]() { return LoopMode == ELoopMode::Finite ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					),
					NAME_None,
					EUserInterfaceActionType::Check
				);
				return MenuBuilder.MakeWidget();
			})
			.ButtonContent()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.WidthOverride(16.0f)
				.HeightOverride(16.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image_Lambda([this]() -> const FSlateBrush* {
						switch (LoopMode) {
							case ELoopMode::None: return FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Loop.Disabled");
							case ELoopMode::Finite: return FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Loop.Finite");
							default: checkNoEntry(); return nullptr;
						}
					})
				]
			];

		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					LoopModeCombo
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SNumericEntryBox<int32>)
					.AllowSpin(true)
					.ToolTipText(LOCTEXT("LoopCountTooltip", "Number of times to loop before stopping. A value of 1 will result in a sequence playing twice before stopping."))
					.Value(TAttribute<TOptional<int32>>::CreateLambda([WeakItem]() { return WeakItem.IsValid() ? WeakItem->NumLoops : TOptional<int32>(); }))
					.Visibility_Lambda([this]() { return LoopMode == ELoopMode::Finite ? EVisibility::Visible : EVisibility::Hidden; })
					.OnValueChanged_Lambda([WeakItem](int32 NewValue) {
						if (WeakItem.IsValid())
						{
							WeakItem->NumLoops = NewValue;
						}
					})
					.OnValueCommitted_Lambda([WeakItem](int32 NewValue, ETextCommit::Type) {
						if (WeakItem.IsValid())
						{
							WeakItem->NumLoops = NewValue;
						}
					})
				]
			]
			+ SVerticalBox::Slot()
			[
				SNullWidget::NullWidget
			];
	}

	if (ColumnName == SSequencerPlaylistPanel::ColumnName_Preload)
	{
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.18"))
				.Text(FEditorFontGlyphs::Circle)
				.ColorAndOpacity_Lambda([this, WeakItem]() {
					USequencerPlaylistsSubsystem* Subsystem = GEditor->GetEditorSubsystem<USequencerPlaylistsSubsystem>();
					USequencerPlaylistItem_Sequence* SeqItem = Cast<USequencerPlaylistItem_Sequence>(WeakItem);
					const bool bHasSession = Subsystem ? Subsystem->GetWeakSession().IsValid() : false;
					if (bHasSession && SeqItem)
					{
						const FTopLevelAssetPath SeqPath(SeqItem->GetSequence());
						TPair<EConcertSequencerPreloadStatus, FText> Status = Subsystem->GetPreloadStatusForSequence(SeqPath);
						switch (Status.Key)
						{
						case EConcertSequencerPreloadStatus::Succeeded: return FSlateColor(EStyleColor::Success);
						case EConcertSequencerPreloadStatus::Pending: return FSlateColor(EStyleColor::Warning);
						}
					}

					return FSlateColor(EStyleColor::Error);
				})
				.ToolTipText_Lambda([this, WeakItem]() {
					USequencerPlaylistsSubsystem* Subsystem = GEditor->GetEditorSubsystem<USequencerPlaylistsSubsystem>();
					USequencerPlaylistItem_Sequence* SeqItem = Cast<USequencerPlaylistItem_Sequence>(WeakItem);
					const bool bHasSession = Subsystem ? Subsystem->GetWeakSession().IsValid() : false;
					if (bHasSession && SeqItem)
					{
						const FTopLevelAssetPath SeqPath(SeqItem->GetSequence());
						TPair<EConcertSequencerPreloadStatus, FText> Status = Subsystem->GetPreloadStatusForSequence(SeqPath);
						return Status.Value;
					}

					return FText::GetEmpty();
				})
			];
	}

	if (ColumnName == SSequencerPlaylistPanel::ColumnName_HoverDetails)
	{
		return SAssignNew(DetailsAnchor, SMenuAnchor)
			.Padding(FMargin(0.0f, 0.0f, 8.0f, 0.0f))
			.Placement(MenuPlacement_MenuLeft)
			.OnGetMenuContent(this, &SSequencerPlaylistItemWidget::EnsureSelectedAndBuildContextMenu)
			[
				SNew(SBox)
				.HeightOverride(24.0f)
				.WidthOverride(18.0f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ContentPadding(0)
					.OnClicked_Lambda([this]() { DetailsAnchor->SetIsOpen(!DetailsAnchor->IsOpen()); return FReply::Handled(); })
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Visibility_Lambda([this]() { return IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FSequencerPlaylistsStyle::Get().GetBrush("SequencerPlaylists.Ellipsis"))
					]
				]
			];
	}

	return SNullWidget::NullWidget;
}


FReply SSequencerPlaylistItemWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		TSharedRef<SWidget> MenuWidget = EnsureSelectedAndBuildContextMenu();
		const FWidgetPath WidgetPath = MouseEvent.GetEventPath() ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuWidget, MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		return FReply::Handled().ReleaseMouseCapture().SetUserFocus(MenuWidget, EFocusCause::SetDirectly);
	}

	return FSuperRowType::OnMouseButtonUp(MyGeometry, MouseEvent);
}


FReply SSequencerPlaylistItemWidget::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<ITypedTableView<TSharedPtr<FSequencerPlaylistRowData>>> OwnerTable = OwnerTablePtr.Pin();
	if (OwnerTable)
	{
		return FReply::Handled().BeginDragDrop(FSequencerPlaylistItemDragDropOp::New(OwnerTable->GetSelectedItems()));
	}

	return FReply::Unhandled();
}


void SSequencerPlaylistItemWidget::SetLoopMode(ELoopMode InLoopMode)
{
	if (LoopMode == InLoopMode)
	{
		return;
	}

	LoopMode = InLoopMode;

	if (InLoopMode == ELoopMode::None)
	{
		USequencerPlaylistItem* Item = RowData->WeakItem.Get();
		if (ensure(Item))
		{
			Item->NumLoops = 0;
		}
	}
}


EVisibility SSequencerPlaylistItemWidget::GetRowDimmingVisibility() const
{
	if (GetPlayModeTransportVisibility() == EVisibility::Visible)
	{
		return EVisibility::Visible;
	}

	if (USequencerPlaylistItem* Item = RowData->WeakItem.Get())
	{
		return Item->bMute ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden;
	}

	return EVisibility::Hidden;
}


EVisibility SSequencerPlaylistItemWidget::GetPlayModeTransportVisibility() const
{
	if (IsHovered() && InPlayMode())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}


TOptional<int32> SSequencerPlaylistItemWidget::GetItemLengthDisplayFrames() const
{
	if (const USequencerPlaylistItem_Sequence* SequenceItem = Cast<const USequencerPlaylistItem_Sequence>(GetItem()))
	{
		if (SequenceItem->GetSequence() && SequenceItem->GetSequence()->GetMovieScene())
		{
			UMovieScene* MovieScene = SequenceItem->GetSequence()->GetMovieScene();
			TRange<FFrameNumber> Range = MovieScene->GetPlaybackRange();
			if (Range.GetLowerBound().IsClosed() && Range.GetUpperBound().IsClosed())
			{
				return ConvertFrameTime(Range.Size<FFrameTime>(),
					MovieScene->GetTickResolution(),
					MovieScene->GetDisplayRate()).FloorToFrame().Value;
			}
		}
	}

	return 0;
}


TSharedRef<SWidget> SSequencerPlaylistItemWidget::EnsureSelectedAndBuildContextMenu()
{
	TArray<UObject*> SelectedItems;

	TSharedPtr<ITypedTableView<TSharedPtr<FSequencerPlaylistRowData>>> OwnerTable = OwnerTablePtr.Pin();
	if (OwnerTable)
	{
		TArray<TSharedPtr<FSequencerPlaylistRowData>> SelectedRows = OwnerTable->GetSelectedItems();
		SelectedItems.Reserve(SelectedRows.Num());
		for (const TSharedPtr<FSequencerPlaylistRowData>& SelectedRow : SelectedRows)
		{
			SelectedItems.Add(SelectedRow->WeakItem.Get());
		}

		if (!SelectedItems.Contains(GetItem()))
		{
			OwnerTable->Private_ClearSelection();
			OwnerTable->Private_SetItemSelection(RowData, true, true);
			OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
			SelectedItems.Empty(1);
			SelectedItems.Add(GetItem());
		}
	}
	else
	{
		SelectedItems.Add(GetItem());
	}

	return BuildContextMenu(SelectedItems);
}


TSharedRef<SWidget> SSequencerPlaylistItemWidget::BuildContextMenu(const TArray<UObject*>& SelectedItems)
{
	FSequencerPlaylistsModule& Module = static_cast<FSequencerPlaylistsModule&>(ISequencerPlaylistsModule::Get());
	FMenuBuilder MenuBuilder(true, Module.GetCommandList());

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ItemContextPlaybackHeading", "Playback"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ItemContextPlay", "Play"),
			PlayItemTooltipText,
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() { PlayClickedDelegate.Execute(*this); }))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ItemContextStop", "Stop"),
			StopItemTooltipText,
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() { StopClickedDelegate.Execute(*this); }))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ItemContextReset", "Reset"),
			ResetItemTooltipText,
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() { ResetClickedDelegate.Execute(*this); }))
		);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ItemContextEditHeading", "Edit"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ItemContextRemove", "Remove from Playlist"),
			LOCTEXT("ItemContextRemoveTooltip", "Remove this item from the Playlist"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.Delete"),
			FUIAction(FExecuteAction::CreateLambda([this]() { RemoveClickedDelegate.Execute(*this); }))
		);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ItemContextDetailsHeading", "Details"));
	{
		FDetailsViewArgs DetailsViewArgs;
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bCustomFilterAreaLocation = true;
			DetailsViewArgs.bCustomNameAreaLocation = true;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bSearchInitialKeyFocus = true;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = false;
			DetailsViewArgs.ColumnWidth = 0.45f;
		}

		TSharedRef<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
		DetailsView->SetIsPropertyVisibleDelegate(IsPropertyVisibleDelegate);
		DetailsView->SetObjects(SelectedItems);
		MenuBuilder.AddWidget(DetailsView, FText::GetEmpty(), true);
	}

	return MenuBuilder.MakeWidget();
}


#undef LOCTEXT_NAMESPACE
