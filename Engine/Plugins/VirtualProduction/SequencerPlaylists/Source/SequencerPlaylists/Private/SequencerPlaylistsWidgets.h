// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "PropertyEditorDelegates.h"
#include "SequencerPlaylistItem.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h" // IWYU pragma: keep
#include "Widgets/Views/STableRow.h"

class STableViewBase;

struct FAssetData;
class IConcertClientSession;
class SDockTab;
class SHeaderRow;
class SMenuAnchor;
class SMultiLineEditableTextBox;
class SSequencerPlaylistItemWidget;
class SSearchBox;
class STextBlock;
template <typename ItemType> class TTextFilter;
class USequencerPlaylist;
class USequencerPlaylistItem;
class USequencerPlaylistPlayer;


struct FSequencerPlaylistRowData
{
	int32 PlaylistIndex;
	TWeakObjectPtr<USequencerPlaylistItem> WeakItem;

	FSequencerPlaylistRowData(int32 InPlaylistIndex, USequencerPlaylistItem* InItem)
		: PlaylistIndex(InPlaylistIndex)
		, WeakItem(InItem)
	{
	}
};


class SSequencerPlaylistPanel
	: public SCompoundWidget
	, public FSelfRegisteringEditorUndoClient
{
	SLATE_BEGIN_ARGS(SSequencerPlaylistPanel) {}
	SLATE_END_ARGS()

public:
	static const float DefaultWidth;
	static const FName ColumnName_HoverTransport;
	static const FName ColumnName_Items;
	static const FName ColumnName_Offset;
	static const FName ColumnName_Pause;
	static const FName ColumnName_Loop;
	static const FName ColumnName_Preload;
	static const FName ColumnName_HoverDetails;

	void Construct(const FArguments& InArgs, TSharedPtr<SDockTab> InContainingTab);
	virtual ~SSequencerPlaylistPanel();

	bool InPlayMode() const { return bPlayMode; }

	FString GetDisplayTitle();
	void SetPlayer(USequencerPlaylistPlayer* Player);
	void LoadPlaylist(USequencerPlaylist* Playlist);

	//~ Begin FEditorUndoClient interface
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient interface

private:
	TSharedRef<SWidget> Construct_LeftToolbar();
	TSharedRef<SWidget> Construct_RightToolbar();
	TSharedRef<SWidget> Construct_Transport();
	TSharedRef<SWidget> Construct_AddSearchRow();
	TSharedRef<SWidget> Construct_ItemListView();

	USequencerPlaylist* GetCheckedPlaylist();
	USequencerPlaylist* GetPlaylist();
	void RegenerateRows();

	TSharedRef<SWidget> BuildOpenPlaylistMenu();
	void OnSavePlaylist();
	void OnSavePlaylistAs();
	void SavePlaylistAs(const FString& PackageName);
	void OnLoadPlaylist(const FAssetData& InPreset);
	void OnNewPlaylist();

	void GetSearchStrings(const FSequencerPlaylistRowData& Item, TArray<FString>& OutSearchStrings);
	void OnSearchTextChanged(const FText& InFilterText);

	FReply HandleClicked_PlayAll();
	FReply HandleClicked_PlayAllReverse();
	FReply HandleClicked_PauseAll();
	FReply HandleClicked_StopAll();
	FReply HandleClicked_ResetAll();
	FReply HandleClicked_AddSequence();

	FReply HandleClicked_Item_Play(SSequencerPlaylistItemWidget& ItemWidget);
	FReply HandleClicked_Item_PlayReverse(SSequencerPlaylistItemWidget& ItemWidget);
	FReply HandleClicked_Item_Pause(SSequencerPlaylistItemWidget& ItemWidget);
	FReply HandleClicked_Item_Stop(SSequencerPlaylistItemWidget& ItemWidget);
	FReply HandleClicked_Item_Reset(SSequencerPlaylistItemWidget& ItemWidget);
	FReply HandleClicked_Item_Remove(SSequencerPlaylistItemWidget& ItemWidget);

	bool HandleItemDetailsIsPropertyVisible(const FPropertyAndParent& PropertyAndParent);

	// Drag and drop handling
	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FSequencerPlaylistRowData> RowData);
	FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FSequencerPlaylistRowData> RowData);

	// Multi-user event handling
	void OnConcertSessionStartup(TSharedRef<IConcertClientSession> InSession);
	void OnConcertSessionShutdown(TSharedRef<IConcertClientSession> InSession);

private:
	TWeakObjectPtr<USequencerPlaylistPlayer> WeakPlayer;
	TWeakObjectPtr<USequencerPlaylist> WeakLoadedPlaylist;

	bool bPlayMode = false;

	TWeakPtr<SDockTab> WeakContainingTab;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<TTextFilter<const FSequencerPlaylistRowData&>> SearchTextFilter;
	TSharedPtr<SHeaderRow> HeaderRow;
	TArray<TSharedPtr<FSequencerPlaylistRowData>> ItemRows;
	TSharedPtr<SListView<TSharedPtr<FSequencerPlaylistRowData>>> ItemListView;
};


class FSequencerPlaylistItemDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FSequencerPlaylistItemDragDropOp, FDragDropOperation)

	TArray<TSharedPtr<FSequencerPlaylistRowData>> SelectedItems;

	static TSharedRef<FSequencerPlaylistItemDragDropOp> New(const TArray<TSharedPtr<FSequencerPlaylistRowData>>& InSelectedItems);

public:
	virtual ~FSequencerPlaylistItemDragDropOp();

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return Decorator;
	}

private:
	TSharedPtr<SWidget> Decorator;
};


DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnClickedSequencerPlaylistItem, SSequencerPlaylistItemWidget& /*ItemWidget*/);


class SSequencerPlaylistItemWidget : public SMultiColumnTableRow<TSharedPtr<FSequencerPlaylistRowData>>
{
	static const FText PlayReverseItemTooltipText;
	static const FText PlayItemTooltipText;
	static const FText PauseItemTooltipText;
	static const FText StopItemTooltipText;
	static const FText ResetItemTooltipText;

	SLATE_BEGIN_ARGS(SSequencerPlaylistItemWidget) {}
		SLATE_ATTRIBUTE(bool, PlayMode)
		SLATE_ATTRIBUTE(bool, IsPlaying)
		SLATE_ATTRIBUTE(bool, IsPaused)

		SLATE_EVENT(FOnClickedSequencerPlaylistItem, OnPlayClicked)
		SLATE_EVENT(FOnClickedSequencerPlaylistItem, OnPlayReverseClicked)
		SLATE_EVENT(FOnClickedSequencerPlaylistItem, OnPauseClicked)
		SLATE_EVENT(FOnClickedSequencerPlaylistItem, OnStopClicked)
		SLATE_EVENT(FOnClickedSequencerPlaylistItem, OnResetClicked)
		SLATE_EVENT(FOnClickedSequencerPlaylistItem, OnRemoveClicked)

		SLATE_EVENT(FIsPropertyVisible, OnIsPropertyVisible)
		SLATE_EVENT(FOnCanAcceptDrop, OnCanAcceptDrop)
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)
	SLATE_END_ARGS()

	enum class ELoopMode
	{
		None,
		Finite,
		//Infinite,
	};

public:
	void Construct(const FArguments& InArgs, TSharedPtr<FSequencerPlaylistRowData> InRowData, const TSharedRef<STableViewBase>& OwnerTableView);

	const TSharedPtr<FSequencerPlaylistRowData>& GetRowData() { return RowData; }
	TSharedPtr<const FSequencerPlaylistRowData> GetRowData() const { return RowData; }

	USequencerPlaylistItem* GetItem() { return GetRowData() ? GetRowData()->WeakItem.Get() : nullptr; }
	const USequencerPlaylistItem* GetItem() const { return GetRowData() ? GetRowData()->WeakItem.Get() : nullptr; }

	//~ Begin STableRow
	void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override;
	//~ End STableRow

	//~ Begin SMultiColumnTableRow
	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	//~ End SMultiColumnTableRow

	//~ Begin SWidget
	FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget

private:
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	bool InPlayMode() const { return PlayMode.Get(); }
	void SetLoopMode(ELoopMode InLoopMode);

	EVisibility GetRowDimmingVisibility() const;
	EVisibility GetPlayModeTransportVisibility() const;
	TOptional<int32> GetItemLengthDisplayFrames() const;

	TSharedRef<SWidget> EnsureSelectedAndBuildContextMenu();
	TSharedRef<SWidget> BuildContextMenu(const TArray<UObject*>& SelectedItems);

private:
	TSharedPtr<FSequencerPlaylistRowData> RowData;
	TSharedPtr<SMenuAnchor> DetailsAnchor;

	ELoopMode LoopMode;

	TAttribute<bool> PlayMode;
	TAttribute<bool> IsPlaying;
	TAttribute<bool> IsPaused;

	FOnClickedSequencerPlaylistItem PlayClickedDelegate;
	FOnClickedSequencerPlaylistItem PlayReverseClickedDelegate;
	FOnClickedSequencerPlaylistItem PauseClickedDelegate;
	FOnClickedSequencerPlaylistItem StopClickedDelegate;
	FOnClickedSequencerPlaylistItem ResetClickedDelegate;
	FOnClickedSequencerPlaylistItem RemoveClickedDelegate;
	FIsPropertyVisible IsPropertyVisibleDelegate;
};
