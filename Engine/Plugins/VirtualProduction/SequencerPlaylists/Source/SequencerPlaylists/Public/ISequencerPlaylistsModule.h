// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleManager.h"
#include "Templates/SubclassOf.h"
#include "ISequencerPlaylistsModule.generated.h"


class ISequencer;
class USequencerPlaylistItem;
class USequencerPlaylistPlayer;


UENUM(BlueprintType)
enum class ESequencerPlaylistPlaybackDirection : uint8
{
	Forward,
	Reverse,
};


USTRUCT(BlueprintType)
struct FSequencerPlaylistPlaybackState
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category="Sequencer Playlists")
	uint8 bIsPlaying : 1;

	UPROPERTY(BlueprintReadWrite, Category="Sequencer Playlists")
	uint8 bIsPaused : 1;

	UPROPERTY(BlueprintReadWrite, Category="Sequencer Playlists")
	ESequencerPlaylistPlaybackDirection PlaybackDirection;

	FSequencerPlaylistPlaybackState()
		: bIsPlaying(false)
		, bIsPaused(false)
		, PlaybackDirection(ESequencerPlaylistPlaybackDirection::Forward)
	{
	}
};


/** Interface for derived classes that play one or more subclasses of USequencerPlaylistItem. */
class ISequencerPlaylistItemPlayer
{
public:
	virtual ~ISequencerPlaylistItemPlayer() {}

	/**
	 * Initiate playback of the specified item.
	 * @return True if the current sequence was modified, otherwise false.
	 */
	virtual bool Play(USequencerPlaylistItem* Item,
	                  ESequencerPlaylistPlaybackDirection Direction = ESequencerPlaylistPlaybackDirection::Forward) = 0;

	/**
	 * Suspend playback of the specified item (if it is playing) and begin a hold.
	 * If it is already paused, resume playback.
	 * @return True if the current sequence was modified, otherwise false.
	 */
	virtual bool TogglePause(USequencerPlaylistItem* Item) = 0;

	/**
	 * Halt any current playback of the specified item.
	 * @return True if the current sequence was modified, otherwise false.
	 */
	virtual bool Stop(USequencerPlaylistItem* Item) = 0;

	/**
	 * Adds a first frame hold section for the specified item.
	 * @return True if the current sequence was modified, otherwise false.
	 */
	virtual bool AddHold(USequencerPlaylistItem* Item) = 0;

	/**
	 * Reset the specified item.
	 * This is essentially a Stop(), followed by an AddHold() if applicable.
	 * @return True if the current sequence was modified, otherwise false.
	 */
	virtual bool Reset(USequencerPlaylistItem* Item) = 0;

	virtual FSequencerPlaylistPlaybackState GetPlaybackState(USequencerPlaylistItem* Item) const = 0;

	/** Returns whether we're currently playing the specified item. */
	UE_DEPRECATED(5.2, "Use GetPlaybackState(Item).bIsPlaying instead.")
	virtual bool IsPlaying(USequencerPlaylistItem* Item) const
	{
		return GetPlaybackState(Item).bIsPlaying;
	}
};


DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<ISequencerPlaylistItemPlayer>, FSequencerPlaylistItemPlayerFactory, TSharedRef<ISequencer>);


class ISequencerPlaylistsModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static ISequencerPlaylistsModule& Get()
	{
		static const FName ModuleName = "SequencerPlaylists";
		return FModuleManager::LoadModuleChecked<ISequencerPlaylistsModule>(ModuleName);
	}

	virtual bool RegisterItemPlayer(TSubclassOf<USequencerPlaylistItem> ItemClass, FSequencerPlaylistItemPlayerFactory PlayerFactory) = 0;

	UE_DEPRECATED(5.1, "There is no longer a \"default\" player. Open a specific Playlist asset to create a player associated with it.")
	virtual USequencerPlaylistPlayer* GetDefaultPlayer() { return nullptr; }
};
