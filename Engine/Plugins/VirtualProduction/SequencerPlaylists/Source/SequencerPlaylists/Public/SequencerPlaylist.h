// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SequencerPlaylist.generated.h"


class USequencerPlaylistItem;


UCLASS(BlueprintType)
class USequencerPlaylist : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, BlueprintReadWrite, Category="SequencerPlaylists")
	FText Description;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="SequencerPlaylists")
	TArray<TObjectPtr<USequencerPlaylistItem>> Items;
};
