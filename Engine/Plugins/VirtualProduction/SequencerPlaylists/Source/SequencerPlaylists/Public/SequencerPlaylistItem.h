// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SequencerPlaylistItem.generated.h"


UCLASS(BlueprintType, Abstract, Within=SequencerPlaylist)
class USequencerPlaylistItem : public UObject
{
	GENERATED_BODY()

public:
	virtual FText GetDisplayName() PURE_VIRTUAL(USequencerPlaylistItem::GetDisplayName, return FText::GetEmpty(); )

public:
	/** Number of frames by which to clip the in point of sections played from this item. Will also affect the first frame for hold. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Sequencer Playlists")
	int32 StartFrameOffset;

	/** Number of frames by which to clip the out point of sections played from this item. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Sequencer Playlists")
	int32 EndFrameOffset;

	/**
	 * If true, the sequence will be inserted immediately on recording start and any time Reset()
	 * is called, paused at the first frame indefinitely until subsequently played or stopped.
	 * This is controlled by toggling pause prior to recording/playback.
	 */
	UPROPERTY()
	bool bHoldAtFirstFrame;

	/** 0 is single playthrough, >= 1 is (n+1) playthroughs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Sequencer Playlists")
	int32 NumLoops;

	/** Speed multiplier at which to play sections created by this item. A value of 1 will result in playback at the original speed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Sequencer Playlists")
	float PlaybackSpeed = 1.0f;

	/** Disable playback of this item and prevent it from entering a hold state. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Sequencer Playlists")
	bool bMute;
};
