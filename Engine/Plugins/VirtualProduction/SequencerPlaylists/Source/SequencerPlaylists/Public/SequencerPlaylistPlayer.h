// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISequencerPlaylistsModule.h"
#include "SequencerPlaylistItem.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "Tickable.h"

#include "SequencerPlaylistPlayer.generated.h"


class ISequencer;
class ISequencerPlaylistItemPlayer;
class USequencerPlaylist;
class USequencerPlaylistPlayer;
class UTakeRecorder;
class FTickablePlaylist;


DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPlayerSequencerPlaylistSet, USequencerPlaylistPlayer*, Player, USequencerPlaylist*, Playlist);


/** Controls playback of playlist items */
UCLASS(BlueprintType)
class USequencerPlaylistPlayer : public UObject
{
	GENERATED_BODY()

public:
	USequencerPlaylistPlayer(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject interface
	void BeginDestroy() override;
	//~ End UObject interface

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	void SetPlaylist(USequencerPlaylist* InPlaylist);

	UFUNCTION(BlueprintPure, Category="Sequencer Playlists")
	USequencerPlaylist* GetPlaylist() { return Playlist; }

	UE_DEPRECATED(5.1, "There is no longer a \"default\" player. Open a specific Playlist asset to create a player associated with it.")
	UFUNCTION(BlueprintPure, meta=(DeprecatedFunction, DeprecationMessage= "There is no longer a \"default\" player. Open a specific Playlist asset to create a player associated with it.", DisplayName="Get Default Sequencer Playlist Player"), Category="SequencerPlaylists")
	static USequencerPlaylistPlayer* GetDefaultPlayer() { return nullptr; }

	UPROPERTY(BlueprintAssignable, Category="Sequencer Playlists")
	FOnPlayerSequencerPlaylistSet OnPlaylistSet;

public:
	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	bool PlayItem(USequencerPlaylistItem* Item,
	              ESequencerPlaylistPlaybackDirection Direction = ESequencerPlaylistPlaybackDirection::Forward);

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	bool PauseItem(USequencerPlaylistItem* Item);

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	bool StopItem(USequencerPlaylistItem* Item);

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	bool ResetItem(USequencerPlaylistItem* Item);

	UE_DEPRECATED(5.2, "Call GetPlaybackState and use the bIsPlaying flag in FSequencerPlaylistPlaybackState instead.")
	UFUNCTION(BlueprintPure, Category="Sequencer Playlists")
	bool IsPlaying(USequencerPlaylistItem* Item);

	UFUNCTION(BlueprintPure, Category="Sequencer Playlists")
	FSequencerPlaylistPlaybackState GetPlaybackState(USequencerPlaylistItem* Item);

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	bool PlayAll(ESequencerPlaylistPlaybackDirection Direction = ESequencerPlaylistPlaybackDirection::Forward);

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	bool PauseAll();

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	bool StopAll();

	UFUNCTION(BlueprintCallable, Category="Sequencer Playlists")
	bool ResetAll();

private:
	TSharedPtr<ISequencer> GetOrCreateSequencer();

	/**
	 * Enters unbounded play (like Take Recorder) if the user is not recording.
	 */
	void EnterUnboundedPlayIfNotRecording();

	/**
	 * Ticked by a tickable game object to perform any necessary time-sliced logic
	 */
	void Tick(float DeltaTime);

	void OnTakeRecorderInitialized(UTakeRecorder* InRecorder);
	void OnTakeRecorderStarted(UTakeRecorder* InRecorder);
	void OnTakeRecorderStopped(UTakeRecorder* InRecorder);

	TSharedPtr<ISequencerPlaylistItemPlayer> GetCheckedItemPlayer(USequencerPlaylistItem* Item);

	friend class FTickablePlaylist;

private:
	TMap<TSubclassOf<USequencerPlaylistItem>, TSharedRef<ISequencerPlaylistItemPlayer>> ItemPlayersByType;

	UPROPERTY(Transient)
	TObjectPtr<USequencerPlaylist> Playlist;

	/** The last initialized recorder, and the one we've bound delegates on. */
	TWeakObjectPtr<UTakeRecorder> WeakRecorder;

	TWeakPtr<ISequencer> WeakSequencer;

	/** A tickable object that will do the playing in the event we are not using a take recorder. */
	TUniquePtr<FTickablePlaylist> PlaylistTicker;
};


class FTickablePlaylist : public FTickableGameObject
{
public:
	FTickablePlaylist(USequencerPlaylistPlayer *InPlayer)
		: WeakPlaylistPlayer(InPlayer)
	{}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FTickablePlaylist, STATGROUP_Tickables);
	}

	// Make sure it always ticks, otherwise we can miss recording, particularly when time code is always increasing throughout the system.
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

	virtual bool IsTickableInEditor() const override
	{
		return true;
	}

	virtual UWorld* GetTickableGameObjectWorld() const override
	{
		USequencerPlaylistPlayer* Player = WeakPlaylistPlayer.Get();
		return Player ? Player->GetWorld() : nullptr;
	}

	virtual void Tick(float DeltaTime) override
	{
		if (USequencerPlaylistPlayer* Player = WeakPlaylistPlayer.Get())
		{
			Player->Tick(DeltaTime);
		}
	}

private:
	TWeakObjectPtr<USequencerPlaylistPlayer> WeakPlaylistPlayer;

};
