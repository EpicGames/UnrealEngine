// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SequencerPlaylists : ModuleRules
{
	public SequencerPlaylists(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[] {
			"Core",
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Projects",
			"InputCore",
			"EditorFramework",
			"UnrealEd",
			"ToolMenus",
			"CoreUObject",
			"Engine",
			"Slate",
			"SlateCore",
			"WorkspaceMenuStructure",
			"LevelSequence",
			"TakeRecorder",
			"PropertyEditor",
			"EditorStyle",
			"MovieScene",
			"LevelSequenceEditor",
			"TakesCore",
			"ToolWidgets",
			"MovieSceneTools",
			"EditorSubsystem",
			"ConcertClient",
			"ConcertSyncClient",
			"ConcertSyncCore",
		});

		DynamicallyLoadedModuleNames.AddRange(new string[] {
		});
	}
}
